{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "1.0",
  "FriendlyName": "Code Quality Tests Unreal Test Plugin",
  "Description": "Tests for the CQTest Unreal Engine module",
  "Category": "Testing",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "DocsURL": "",
  "MarketplaceURL": "",
  "SupportURL": "",
  "CanContainContent": false,
  "IsBetaVersion": false,
  "Installed": false,
  "Modules": [
		{
			"Name": "CQTestTests",
			"Type": "DeveloperTool",
			"LoadingPhase": "Default"
		}
	]
}