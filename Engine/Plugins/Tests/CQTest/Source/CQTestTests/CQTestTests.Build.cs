// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CQTestTests : ModuleRules
{
	public CQTestTests(ReadOnlyTargetRules Target)
		: base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"Slate",
					"CQTest"
				 }
			);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetTools",
					"EngineSettings",
					"LevelEditor",
					"UnrealEd"
			});
		}

		SetupIrisSupport(Target);
	}
}
