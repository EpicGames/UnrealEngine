// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "Assert/NoDiscardAsserter.h"

DECLARE_LOG_CATEGORY_CLASS(AssertionTests, Log, All);

namespace CQTests
{
	static const FString ExpectedError = TEXT("Expected");
	static const TCHAR* AnyError = TEXT("");

	TEST_CLASS(NoDiscardAssert_Errors, "TestFramework.CQTest.Core")
	{
		FNoDiscardAsserter Asserter{ *TestRunner };
		TEST_METHOD(AssertFail_WithMessage_AddsError)
		{
			Assert.ExpectError(AnyError);
			Assert.Fail(AnyError);
		}

		PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
		TEST_METHOD(AssertFail_Macro_AddsErrorAndExits)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_FAIL(ExpectedError);
			ASSERT_FAIL(TEXT("It should not assert after the first expected assert since it exits"));
		}
		PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS

		TEST_METHOD(Assertions_Accept_RawStrings)
		{
			Asserter.ExpectError("Hello World");
			Asserter.Fail("Hello World");
		}

		TEST_METHOD(Assertions_Accept_TCharArrays)
		{
			Asserter.ExpectError(TEXT("Hello World"));
			Asserter.Fail(TEXT("Hello World"));
		}

		TEST_METHOD(Assertions_Accept_FStrings)
		{
			FString message = TEXT("Hello World");
			Asserter.ExpectError(message);
			Asserter.Fail(message);
		}

		TEST_METHOD(AssertExpectError_WithMultipleErrors_Succeeds)
		{
			Asserter.ExpectError(AnyError, 3);
			Asserter.Fail("One");
			Asserter.Fail("Two");
			Asserter.Fail("Three");
		}

		TEST_METHOD(AsserterExpectError_WithZeroExpected_AcceptsAnyNumber)
		{
			Asserter.ExpectError(AnyError, 0);
			Asserter.Fail("One");
			Asserter.Fail("Two");
			Asserter.Fail("Three");
		}

		TEST_METHOD(AssertExpectError_WithMatchingError_Succeeds)
		{
			Asserter.ExpectError("Hello World");
			Asserter.Fail("Hello World");
		}

		TEST_METHOD(AssertExpectError_WithRegexSymbols_EscapesRegex)
		{
			Asserter.ExpectError("[^abc]");
			Asserter.Fail("[^abc]");
		}

		TEST_METHOD(AssertExpectErrorRegex_WithRegex_Succeeds)
		{
			Asserter.ExpectErrorRegex("\\w+");
			Asserter.Fail("abc");
		}
	};

	TEST_CLASS(NoDiscardAssert_Bools, "TestFramework.CQTest.Core")
	{
		TEST_METHOD(AssertTrue_WithTrue_Succeeds)
		{
			ASSERT_THAT(IsTrue(true));
		}
		TEST_METHOD(AssertTrue_WithTrueAndErrorMessage_DoesNotAddErrorMessage)
		{
			ASSERT_THAT(IsTrue(true, "Unexpected"));
		}
		TEST_METHOD(AssertTrue_WithFalse_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsTrue(false));
		}
		TEST_METHOD(AssertTrue_WithFalseAndError_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(IsTrue(false, ExpectedError));
		}

		TEST_METHOD(AssertFalse_WithFalse_Succeeds)
		{
			ASSERT_THAT(IsFalse(false));
		}
		TEST_METHOD(AssertFalse_WithFalseAndErrorMessage_DoesNotAddErrorMessage)
		{
			ASSERT_THAT(IsFalse(false, "Unexpected"));
		}
		TEST_METHOD(AssertFalse_WithTrue_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsFalse(true));
		}
		TEST_METHOD(AssertFalse_WithTrueAndError_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(IsFalse(true, ExpectedError));
		}
	};

	TEST_CLASS(NoDiscardAssert_Numbers, "TestFramework.CQTest.Core")
	{
		TEST_METHOD(AssertEqual_WithSameInts_Succeeds)
		{
			ASSERT_THAT(AreEqual(42, 42));
		}
		TEST_METHOD(AssertEqual_WithSameIntsAndErrorMessage_DoesNotAddErrorMessage)
		{
			ASSERT_THAT(AreEqual(42, 42, "Unexpected"));
		}
		TEST_METHOD(AssertEqual_WithDifferentInts_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreEqual(42, 0));
		}
		TEST_METHOD(AssertEqual_WithDifferentIntsAndErrorMessage_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(AreEqual(42, 0, ExpectedError));
		}

		/* These cases intentionally do not compile to avoid accidental floating point error
		TEST_METHOD(AssertEqual_WithDoubles_CompileError)
		{
			ASSERT_THAT(AreEqual(42.0, 42.0));
		}
		TEST_METHOD(AssertEqual_WithFloats_CompileError)
		{
			ASSERT_THAT(AreEqual(42.f, 42.f));
		}
		TEST_METHOD(AssertEqual_WithMixedFloat_CompileError)
		{
			ASSERT_THAT(AreEqual(42, 42.f));
		}

		TEST_METHOD(AssertNotEqual_WithDoubles_CompileError)
		{
			ASSERT_THAT(AreNotEqual(42.0, 42.0));
		}
		TEST_METHOD(AssertNotEqual_WithFloats_CompileError)
		{
			ASSERT_THAT(AreNotEqual(42.f, 42.f));
		}
		TEST_METHOD(AssertNotEqual_WithMixedFloat_CompileError)
		{
			ASSERT_THAT(AreNotEqual(42, 42.f));
		}
		*/

		TEST_METHOD(AssertNotEqual_WithDifferentInts_Succeeds)
		{
			ASSERT_THAT(AreNotEqual(42, 0));
		}
		TEST_METHOD(AssertNotEqual_WithDifferentAndErrorMessage_DoesNotAddErrorMessage)
		{
			ASSERT_THAT(AreNotEqual(42, 0, "Unexpected"));
		}
		TEST_METHOD(AssertNotEqual_WithSameInts_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreNotEqual(42, 42));
		}
		TEST_METHOD(AssertNotEqual_WithSameIntsAndErrorMessage_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(AreNotEqual(42, 42, ExpectedError));
		}

		TEST_METHOD(AssertNear_WithSameDoubles_Succeeds)
		{
			ASSERT_THAT(IsNear(3.14, 3.14, 0.001));
		}
		TEST_METHOD(AssertNear_WithSameDoublesAndErrorMessage_DoesNotAddErrorMessage)
		{
			ASSERT_THAT(IsNear(3.14, 3.14, 0.001, "Unexpected"));
		}
		TEST_METHOD(AssertNear_WithSimilarDoubles_Succeeds)
		{
			ASSERT_THAT(IsNear(3.0, 3.1, 1.0));
		}
		TEST_METHOD(AssertNear_WithDifferentDoubles_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsNear(1.0, 2.0, 0.001));
		}
		TEST_METHOD(AssertNear_WithDifferentDoublesAndError_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(IsNear(1.0, 2.0, 0.001, ExpectedError));
		}
		TEST_METHOD(AssertNear_WithSimilarDoubleWithinTolerance_Succeeds)
		{
			ASSERT_THAT(IsNear(3.0, 3.001, 0.01));
		}
		TEST_METHOD(AssertNear_WithSimilarDoubleWithinNegativeTolerance_Succeeds)
		{
			ASSERT_THAT(IsNear(3.0, 2.999, 0.01));
		}
		TEST_METHOD(AssertNear_WithSimilarDoubleOutsideTolerance_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsNear(3.0, 3.1, 0.01));
		}
		TEST_METHOD(AssertNear_WithSimilarDoubleOutsideNegativeTolerance_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsNear(3.0, 2.9, 0.01));
		}
	};

	TEST_CLASS(NoDiscardAssert_Pointers, "TestFramework.CQTest.Core")
	{
		TEST_METHOD(AssertNull_WithNullPtr_Succeeds)
		{
			int* ptr = nullptr;
			ASSERT_THAT(IsNull(ptr));
		}
		TEST_METHOD(AssertNull_WithNullPtrAndErrorMessage_DoesNotAddError)
		{
			int* ptr = nullptr;
			ASSERT_THAT(IsNull(ptr, "Unexpected"));
		}
		TEST_METHOD(AssertNull_WithNonNull_AddsError)
		{
			int val = 42;
			int* ptr = &val;
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsNull(ptr));
		}
		TEST_METHOD(AssertNull_WithNonNullAndErrorMessage_AddsSpecificErrorMessage)
		{
			int val = 42;
			int* ptr = &val;
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(IsNull(ptr, ExpectedError));
		}
		TEST_METHOD(AssertNull_WithInvalidSharedPtr_Succeeds)
		{
			auto ptr = TSharedPtr<int>{};
			ASSERT_THAT(IsNull(ptr));
		}
		TEST_METHOD(AssertNull_WithValidSharedPtr_AddsError)
		{
			TSharedPtr<int> ptr = MakeShared<int>(42);
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsNull(ptr));
		}
		TEST_METHOD(AssertNull_WithInvalidUniquePtr_Succeeds)
		{
			TUniquePtr<int> ptr;
			ASSERT_THAT(IsNull(ptr));
		}
		TEST_METHOD(AssertNull_WithValidUniquePtr_AddsError)
		{
			TUniquePtr<int> ptr = MakeUnique<int>(42);
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsNull(ptr));
		}

		TEST_METHOD(AssertNotNull_WithNullPtr_AddsError)
		{
			int* ptr = nullptr;
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsNotNull(ptr));
		}
		TEST_METHOD(AssertNotNull_WithNullPtrAndError_AddsSpecificError)
		{
			int* ptr = nullptr;
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(IsNotNull(ptr, ExpectedError));
		}
		TEST_METHOD(AssertNotNull_WithNonNull_Succeeds)
		{
			int val = 42;
			int* ptr = &val;
			ASSERT_THAT(IsNotNull(ptr));
		}
		TEST_METHOD(AssertNotNull_WithNonNullAndErrorMessage_DoesNotAddErrorMessage)
		{
			int val = 42;
			int* ptr = &val;
			ASSERT_THAT(IsNotNull(ptr, "Unexpected"));
		}
		TEST_METHOD(AssertNotNull_WithInvalidSharedPtr_AddsError)
		{
			auto ptr = TSharedPtr<int>{};
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsNotNull(ptr));
		}
		TEST_METHOD(AssertNotNull_WithValidSharedPtr_Succeeds)
		{
			TSharedPtr<int> ptr = MakeShared<int>(42);
			ASSERT_THAT(IsNotNull(ptr));
		}
		TEST_METHOD(AssertNotNull_WithInvalidUniquePtr_AddsError)
		{
			TUniquePtr<int> ptr;
			Assert.ExpectError(AnyError);
			ASSERT_THAT(IsNotNull(ptr));
		}
		TEST_METHOD(AssertNotNull_WithValidUniquePtr_Succeeds)
		{
			TUniquePtr<int> ptr = MakeUnique<int>(42);
			ASSERT_THAT(IsNotNull(ptr));
		}
	};

	TEST_CLASS(NoDiscardAssert_Strings, "TestFramework.CQTest.Core")
	{
		const TCHAR* SomeText{ TEXT("Hello") };
		const TCHAR* OtherText{ TEXT("World") };

		const TCHAR* UpperText{ TEXT("ONE") };
		const TCHAR* LowerText{ TEXT("one") };

		const FString SomeString{ SomeText };
		const FString OtherString{ OtherText };

		TEST_METHOD(AssertAreEqual_Text_Matching_Succeeds)
		{
			ASSERT_THAT(AreEqual(SomeText, SomeText));
		}
		TEST_METHOD(AssertAreEqual_Text_Mismatch_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreEqual(SomeText, OtherText));
		}
		TEST_METHOD(AssertAreEqual_Text_DifferentCases_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreEqual(UpperText, LowerText));
		}
		TEST_METHOD(AssertAreEqual_Text_MismatchAndError_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(AreEqual(SomeText, OtherText, ExpectedError));
		}
		TEST_METHOD(AssertAreEqual_FString_Matching_Succeeds)
		{
			ASSERT_THAT(AreEqual(SomeString, SomeString));
		}
		TEST_METHOD(AssertAreEqual_FString_Mismatch_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreEqual(SomeString, OtherString));
		}
		TEST_METHOD(AssertAreEqual_FString_MismatchAndError_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(AreEqual(SomeString, OtherString, ExpectedError));
		}
		TEST_METHOD(AssertAreEqual_FString_DifferentCases_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreEqual(SomeString.ToLower(), SomeString.ToUpper()));
		}

		TEST_METHOD(AssertAreEqualIgnoreCase_Text_Matching_Succeeds)
		{
			ASSERT_THAT(AreEqualIgnoreCase(SomeText, SomeText));
		}
		TEST_METHOD(AssertAreEqualIgnoreCase_Text_Mismatch_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreEqualIgnoreCase(SomeText, OtherText));
		}
		TEST_METHOD(AssertAreEqualIgnoreCase_Text_MismatchAndError_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(AreEqualIgnoreCase(SomeText, OtherText, ExpectedError));
		}
		TEST_METHOD(AssertAreEqualIgnoreCase_Text_DifferentCases_Succeeds)
		{
			ASSERT_THAT(AreEqualIgnoreCase(UpperText, LowerText));
		}
		TEST_METHOD(AssertAreEqualIgnoreCase_FString_Matching_Succeeds)
		{
			ASSERT_THAT(AreEqualIgnoreCase(SomeString, SomeString));
		}
		TEST_METHOD(AssertAreEqualIgnoreCase_FString_Mismatch_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreEqualIgnoreCase(SomeString, OtherString));
		}
		TEST_METHOD(AssertAreEqualIgnoreCase_FString_MismatchAndError_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(AreEqualIgnoreCase(SomeString, OtherString, ExpectedError));
		}
		TEST_METHOD(AssertAreEqualIgnoreCase_FString_DifferentCases_Succeeds)
		{
			ASSERT_THAT(AreEqualIgnoreCase(SomeString.ToUpper(), SomeString.ToLower()));
		}

		TEST_METHOD(AssertAreNotEqual_Text_Matching_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreNotEqual(SomeText, SomeText));
		}
		TEST_METHOD(AssertAreNotEqual_Text_MatchingWithError_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(AreNotEqual(SomeText, SomeText, ExpectedError));
		}
		TEST_METHOD(AssertAreNotEqual_Text_Mismatch_Succeeds)
		{
			ASSERT_THAT(AreNotEqual(SomeText, OtherText));
		}
		TEST_METHOD(AssertAreNotEqual_Text_MismatchWithError_Succeeds)
		{
			ASSERT_THAT(AreNotEqual(SomeText, OtherText, "Unexpected"));
		}
		TEST_METHOD(AssertAreNotEqual_Text_DifferentCases_Succeeds)
		{
			ASSERT_THAT(AreNotEqual(UpperText, LowerText));
		}
		TEST_METHOD(AssertAreNotEqual_FString_Matching_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreNotEqual(SomeString, SomeString));
		}
		TEST_METHOD(AssertAreNotEqual_FString_MatchingWithMessage_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(AreNotEqual(SomeString, SomeString, ExpectedError));
		}
		TEST_METHOD(AssertAreNotEqual_FString_Mismatch_Succeeds)
		{
			ASSERT_THAT(AreNotEqual(SomeString, OtherString));
		}
		TEST_METHOD(AssertAreNotEqual_FString_MismatchAndError_Succeeds)
		{
			ASSERT_THAT(AreNotEqual(SomeString, OtherString, ExpectedError));
		}
		TEST_METHOD(AssertAreNotEqual_FString_DifferentCases_Succeeds)
		{
			ASSERT_THAT(AreNotEqual(SomeString.ToLower(), SomeString.ToUpper()));
		}

		TEST_METHOD(AssertAreNotEqualIgnoreCase_Text_Matching_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreNotEqualIgnoreCase(SomeText, SomeText));
		}
		TEST_METHOD(AssertAreNotEqualIgnoreCase_Text_MatchingWithError_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(AreNotEqualIgnoreCase(SomeText, SomeText));
		}
		TEST_METHOD(AssertAreNotEqualIgnoreCase_Text_Mismatch_Succeeds)
		{
			ASSERT_THAT(AreNotEqualIgnoreCase(SomeText, OtherText));
		}
		TEST_METHOD(AssertAreNotEqualIgnoreCase_Text_MismatchAndError_Succeeds)
		{
			ASSERT_THAT(AreNotEqualIgnoreCase(SomeText, OtherText, "Unexpected"));
		}
		TEST_METHOD(AssertAreNotEqualIgnoreCase_Text_DifferentCases_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreNotEqualIgnoreCase(UpperText, LowerText));
		}
		TEST_METHOD(AssertAreNotEqualIgnoreCase_FString_Matching_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreNotEqualIgnoreCase(SomeString, SomeString));
		}
		TEST_METHOD(AssertAreNotEqualIgnoreCase_FString_MatchingWithError_AddsSpecificError)
		{
			Assert.ExpectError(ExpectedError);
			ASSERT_THAT(AreNotEqualIgnoreCase(SomeString, SomeString, ExpectedError));
		}
		TEST_METHOD(AssertAreNotEqualIgnoreCase_FString_Mismatch_Succeeds)
		{
			ASSERT_THAT(AreNotEqualIgnoreCase(SomeString, OtherString));
		}
		TEST_METHOD(AssertAreNotEqualIgnoreCase_FString_MismatchAndError_Succeeds)
		{
			ASSERT_THAT(AreNotEqualIgnoreCase(SomeString, OtherString, "Unexpected"));
		}
		TEST_METHOD(AssertAreNotEqualIgnoreCase_FString_DifferentCases_AddsError)
		{
			Assert.ExpectError(AnyError);
			ASSERT_THAT(AreNotEqualIgnoreCase(SomeString.ToUpper(), SomeString.ToLower()));
		}
	};
	
} // namespace CQTests

struct FCustomType
{
	FCustomType(const FString& InName)
		: Name(InName)
	{
	}

	FString Name;

	bool operator==(const FCustomType& other) const
	{
		return Name == other.Name;
	}
	bool operator!=(const FCustomType& other) const
	{
		return !(*this == other);
	}
};

template <>
FString CQTestConvert::ToString(const FCustomType& obj)
{
	return obj.Name;
}

TEST_CLASS(AssertMessage, "TestFramework.CQTest.Core")
{
	TEST_METHOD(AssertAreEqual_WithUnequalCustomTypes_PrintObjectNames)
	{
		FCustomType left = { "left" };
		FCustomType right = { "right" };

		Assert.ExpectError("Expected left to equal right");
		ASSERT_THAT(AreEqual(left, right));
	}

	TEST_METHOD(AssertAreNotEqual_WithEqualCustomTypes_PrintObjectNames)
	{
		FCustomType obj = { "Object" };

		Assert.ExpectError("Expected Object to not equal Object");
		ASSERT_THAT(AreNotEqual(obj, obj));
	}
};
