// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"

static const FString OverrideStr = FString("Override");
static const FString ToStringStr = FString("ToString");
constexpr auto OStreamStr = "OStream";

struct TypeWithExplicitOverride
{
};

template <>
inline FString CQTestConvert::ToString(const TypeWithExplicitOverride&)
{
	return OverrideStr;
}

struct TypeWithToString
{
	FString ToString() const
	{
		return ToStringStr;
	}
};

struct TypeWithOStream
{
};

CQTestConvert::PrintStream& operator<<(CQTestConvert::PrintStream& stream, const TypeWithOStream& obj)
{
	stream << OStreamStr;
	return stream;
}

struct TypeWithToStringAndOverride
{
	FString ToString() const
	{
		return ToStringStr;
	}
};

template<>
inline FString CQTestConvert::ToString(const TypeWithToStringAndOverride&)
{
	return OverrideStr;
}

struct TypeWithToStringAndOStream
{
	FString ToString() const 
	{
		return ToStringStr;
	}
	CQTestConvert::PrintStream& operator<<(CQTestConvert::PrintStream& stream)
	{
		stream << OStreamStr;
		return stream;
	}
};

enum UnscopedEnumNoOperator
{
	One = 1,
	Two = 2
};

enum UnscopedEnumWithUCharBase : uint8
{
	First = 1,
	Second = 2
};

enum UnscopedEnumWithCharBase : int8
{
	Alpha = 1,
	Beta = 2
};

enum UnscopedEnumWithIntBase : int32
{
	Aleph = 1,
	Bet = 2
};

enum UnscopedEnumWithOperator
{
	Three = 3,
	Four = 4
};

enum struct ScopedEnumNoOperator
{
	One = 1,
	Two = 2
};

enum struct ScopedEnumWithOperator
{
	One = 1,
	Two = 2
};

enum class ScopedEnumWithUCharBase : uint8
{
	One = 1,
	Two = 2
};

enum class ScopedEnumWithCharBase : int8
{
	One = 1,
	Two = 2
};

enum class ScopedEnumWithIntBase : int32
{
	One = 1,
	Two = 2
};

CQTestConvert::PrintStream& operator<<(CQTestConvert::PrintStream& stream, const UnscopedEnumWithOperator& e)
{
	switch (e)
	{
		case UnscopedEnumWithOperator::Three:
			stream << "Three";
			break;
		case UnscopedEnumWithOperator::Four:
			stream << "Four";
			break;
		default:
			stream << "Unknown";
			break;
	}
	return stream;
}

CQTestConvert::PrintStream& operator<<(CQTestConvert::PrintStream& stream, const ScopedEnumWithOperator& e)
{
	switch (e)
	{
		case ScopedEnumWithOperator::One:
			stream << "One";
			break;
		case ScopedEnumWithOperator::Two:
			stream << "Two";
			break;
		default:
			stream << "Unknown";
			break;
	}
	return stream;
}

TEST_CLASS(TestConvertToString, "TestFramework.CQTest.Core")
{
	TEST_METHOD(TypeWithExplicitOverride_ConvertToString_ReturnsOverride)
	{
		TypeWithExplicitOverride obj;
		ASSERT_THAT(AreEqual(OverrideStr, CQTestConvert::ToString(obj)));
	}

	TEST_METHOD(TypeWithToString_ConvertToString_ReturnsToString)
	{
		TypeWithToString obj;
		ASSERT_THAT(AreEqual(ToStringStr, CQTestConvert::ToString(obj)));
	}

	TEST_METHOD(TypeWithOstream_ConvertToString_ReturnsOStream)
	{
		TypeWithOStream obj;
		ASSERT_THAT(AreEqual(FString(OStreamStr), CQTestConvert::ToString(obj)));
	}

	TEST_METHOD(TypeWithToStringAndOverride_ConvertToString_ReturnsOverride)
	{
		TypeWithToStringAndOverride obj;
		ASSERT_THAT(AreEqual(OverrideStr, CQTestConvert::ToString(obj)));
	}

	TEST_METHOD(TypeWithToStringAndOStream_ConvertToString_ReturnsToString)
	{
		TypeWithToStringAndOStream obj;
		ASSERT_THAT(AreEqual(ToStringStr, CQTestConvert::ToString(obj)));
	}

	TEST_METHOD(UnscopedEnumNoOperator_ConvertToString_ReturnsNumber)
	{
		auto e = UnscopedEnumNoOperator::One;
		ASSERT_THAT(AreEqual(FString(TEXT("1")), CQTestConvert::ToString(e)));
	}

	TEST_METHOD(ScopedEnumNoOperator_ConvertToString_ReturnsNumber)
	{
		auto e = ScopedEnumNoOperator::Two;
		ASSERT_THAT(AreEqual(FString(TEXT("2")), CQTestConvert::ToString(e)));
	}

	TEST_METHOD(UnscopedEnumWithOperator_ConvertToString_ReturnsString)
	{
		auto e = UnscopedEnumWithOperator::Three;
		ASSERT_THAT(AreEqual(FString(TEXT("Three")), CQTestConvert::ToString(e)));
	}

	TEST_METHOD(ScopedEnumWithOperator_ConvertToString_ReturnsString)
	{
		auto e = ScopedEnumWithOperator::Two;
		ASSERT_THAT(AreEqual(FString(TEXT("Two")), CQTestConvert::ToString(e)));
	}

	TEST_METHOD(ScopedEnumWithUCharBase_ConverToString_ReturnsNumberAsString)
	{
		auto e = ScopedEnumWithUCharBase::One;
		ASSERT_THAT(AreEqual(FString(TEXT("1")), CQTestConvert::ToString(e)));
	}

	TEST_METHOD(UnscopedEnumWithUCharBase_ConvertToString_ReturnsNumberAsString)
	{
		auto e = UnscopedEnumWithUCharBase::First;
		ASSERT_THAT(AreEqual(FString(TEXT("1")), CQTestConvert::ToString(e)));
	}

	TEST_METHOD(ScopedEnumWithCharBase_ConverToString_ReturnsNumberAsString)
	{
		auto e = ScopedEnumWithCharBase::One;
		ASSERT_THAT(AreEqual(FString(TEXT("1")), CQTestConvert::ToString(e)));
	}

	TEST_METHOD(UnscopedEnumWithCharBase_ConvertToString_ReturnsNumberAsString)
	{
		auto e = UnscopedEnumWithCharBase::Alpha;
		ASSERT_THAT(AreEqual(FString(TEXT("1")), CQTestConvert::ToString(e)));
	}

	TEST_METHOD(ScopedEnumWithIntBase_ConverToString_ReturnsNumberAsString)
	{
		auto e = ScopedEnumWithIntBase::One;
		ASSERT_THAT(AreEqual(FString(TEXT("1")), CQTestConvert::ToString(e)));
	}

	TEST_METHOD(UnscopedEnumWithIntBase_ConvertToString_ReturnsNumberAsString)
	{
		auto e = UnscopedEnumWithIntBase::Aleph;
		ASSERT_THAT(AreEqual(FString(TEXT("1")), CQTestConvert::ToString(e)));
	}

	TEST_METHOD(ArrayOfValues_ConvertToString_ReturnsCommaDelimitedList)
	{
		TArray<int32> values = { 1, 2, 3 };
		ASSERT_THAT(AreEqual(FString(TEXT("[1, 2, 3]")), CQTestConvert::ToString(values)));
	}

	TEST_METHOD(SetOfValues_ConvertToString_ReturnsCommaDelimitedList)
	{
		TSet<int32> values = { 1, 2, 3 };
		auto asStr = CQTestConvert::ToString(values);
		ASSERT_THAT(IsTrue(asStr.Contains(TEXT("1"))));
		ASSERT_THAT(IsTrue(asStr.Contains(TEXT("2"))));
		ASSERT_THAT(IsTrue(asStr.Contains(TEXT("3"))));
	}

	TEST_METHOD(MapOfValues_ConvertToString_ReturnsKeysAndValues)
	{
		TMap<int32, FString> values = { 
			{ 1, FString(TEXT("One")) },
			{ 2, FString(TEXT("Two")) },
			{ 3, FString(TEXT("Three")) } 
		};

		auto asStr = CQTestConvert::ToString(values);
		ASSERT_THAT(IsTrue(asStr.Contains(TEXT("1:One"))));
		ASSERT_THAT(IsTrue(asStr.Contains(TEXT("2:Two"))));
		ASSERT_THAT(IsTrue(asStr.Contains(TEXT("3:Three"))));
	}

	TEST_METHOD(Float_ConvertToString_ReturnsFloatAsString)
	{
		double value = 3.14;
		auto asStr = CQTestConvert::ToString(value);
		ASSERT_THAT(IsTrue(asStr.Contains(TEXT("3.14"))));
	}

	TEST_METHOD(Pointer_ConvertToString_ReturnsPointerAsString)
	{
		int32 Value = 3;
		int32* PtrToValue = &Value;
		ASSERT_THAT(AreEqual(FString::Printf(TEXT("0x%p"), PtrToValue), CQTestConvert::ToString(PtrToValue)));
	}

	TEST_METHOD(OptionalWithValue_ConvertToString_ReturnsValueAsString)
	{
		TOptional<int32> Value(3);
		ASSERT_THAT(AreEqual(FString(TEXT("3")), CQTestConvert::ToString(Value)));
	}

	TEST_METHOD(OptionalWithoutValue_ConvertToString_ReturnsNotSetAsString)
	{
		TOptional<int32> Value;
		ASSERT_THAT(AreEqual(FString(TEXT("[NotSet]")), CQTestConvert::ToString(Value)));
	}
};
