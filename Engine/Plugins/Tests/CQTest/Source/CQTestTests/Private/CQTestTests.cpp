// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "CQTestUnitTestHelper.h"

#include "Tickable.h"

namespace CQTestTests
{
	TEST(Minimal, "TestFramework.CQTest")
	{
		ASSERT_THAT(IsTrue(true));
	};

	TEST_CLASS(From, "TestFramework.CQTest.Core.GenerateDirectory.TokenProduces.[GenerateTestDirectory]")
	{
		TEST_METHOD(FolderStructure)
		{
			FString Expected = TEXT("CQTest");
			ASSERT_THAT(IsTrue(TestRunner->TestDir.EndsWith(Expected), FString::Printf(TEXT("TestDir to end with %s but TestDir is %s produced from %s"), *Expected, *TestRunner->TestDir, *TestRunner->GetTestSourceFileName())));
		}
	};

	TEST_CLASS(Produces, "TestFramework.CQTest.Core.GenerateDirectory")
	{
		TEST_METHOD(WithPlugins_AppearsInPlugins)
		{
			FString GeneratedDirectory = TestDirectoryGenerator::Generate(TEXT("Projects/MyProject/Plugins/PluginOne/Source/Test.cpp"));
			ASSERT_THAT(AreEqual(FString(TEXT("MyProject.Plugins.PluginOne")), GeneratedDirectory));
		}

		TEST_METHOD(WithPlatforms_AppearsInPlatforms)
		{
			FString GeneratedDirectory = TestDirectoryGenerator::Generate(TEXT("Projects/MyProject/Platforms/PlatformOne/Source/Test.cpp"));
			ASSERT_THAT(AreEqual(FString(TEXT("MyProject.Platforms.PlatformOne")), GeneratedDirectory));
		}

		TEST_METHOD(WithoutPluginsOrPlatforms_FallsBackToSource)
		{
			FString GeneratedDirectory = TestDirectoryGenerator::Generate(TEXT("Project/MyProject/Source/MyProjectFolder/Test.cpp"));
			ASSERT_THAT(AreEqual(FString(TEXT("MyProject.Source.MyProjectFolder")), GeneratedDirectory));
		}
	};

	TEST_CLASS(SourceAndFile, "TestFramework.CQTest.Core")
	{
		TEST_METHOD(SetsSourceFile)
		{
			ASSERT_THAT(AreEqual(FString(__FILE__), TestRunner->GetTestSourceFileName()));
		}

		TEST_METHOD(SetsLine_WithLineOfTestClass)
		{
			ASSERT_THAT(AreEqual(__LINE__ - 2, TestRunner->GetTestSourceFileLine(TEXT("SetsLine_WithLineOfTestClass"))));
		}
	};

	TEST_CLASS(DefaultFixtureTestFlags, "TestFramework.CQTest.Core")
	{
		TEST_METHOD(SetsApplicationContextMask)
		{
			ASSERT_THAT(AreEqual(EAutomationTestFlags_ApplicationContextMask, TestRunner->GetTestFlags() & EAutomationTestFlags_ApplicationContextMask));
		}

		TEST_METHOD(SetsProductFilter)
		{
			ASSERT_THAT(AreEqual(EAutomationTestFlags::ProductFilter, TestRunner->GetTestFlags() & EAutomationTestFlags::ProductFilter));
		}
	};

	TEST_CLASS_WITH_FLAGS(OverrideFixtureTestFlags, "TestFramework.CQTest.Core", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
	{
		TEST_METHOD(GetTestFlags_ReturnsSetAutomationTestFlags)
		{
			ASSERT_THAT(AreEqual(EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter, TestRunner->GetTestFlags()));
		}
	};

	TEST_CLASS_WITH_FLAGS_AND_TAGS(OverrideFixtureTestFlagsAndTags, "TestFramework.CQTest.Core.Tags", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter, "[CQFlagsTest][FlagsExtraTag][CQTaggedTests]")
	{
		TEST_METHOD(GetTestFlags_ReturnsSetAutomationTestFlags_HasExpectedTags)
		{
			ASSERT_THAT(AreEqual(EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter, TestRunner->GetTestFlags()));

			const FString TestTags = TestRunner->GetTestTags();
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQFlagsTest]"))));
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[FlagsExtraTag]"))));
		}

		TEST_METHOD_WITH_TAGS(GetTestFlags_HasExpectedMethodTags, "[CQClassWithFlagsAndTagsMethod]")
		{
			const FString ThisMethodName = TEXT("GetTestFlags_HasExpectedMethodTags");
			const FString TestTags = TestRunner->GetTestMethodTags(ThisMethodName);
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQFlagsTest]"))));
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[FlagsExtraTag]"))));
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQClassWithFlagsAndTagsMethod]"))));
		}
	};

	TEST_CLASS(TestFixtureTest, "TestFramework.CQTest.Core")
	{
		bool SetupCalled = false;
		bool ShouldAddErrorDuringTearDown = false;
		uint32 SomeNumber = 0;
		FString ExpectedError = TEXT("Error reported in TearDown");

		BEFORE_EACH()
		{
			SetupCalled = true;
			SomeNumber++;
		}

		AFTER_EACH()
		{
			if (ShouldAddErrorDuringTearDown)
			{
				Assert.Fail(ExpectedError);
			}
		}

	protected:
		void ProtectedMethodDefinedInFixture() const {}

		void AddExpectedErrorDuringTearDown()
		{
			ShouldAddErrorDuringTearDown = true;
			Assert.ExpectError(ExpectedError);
		}

		TEST_METHOD(CanAccessProtectedFixtureMethods)
		{
			ProtectedMethodDefinedInFixture();
		}

		TEST_METHOD(BeforeRunTest_CallsSetup)
		{
			ASSERT_THAT(IsTrue(SetupCalled));
		}

		TEST_METHOD(AfterRunTest_CallsTearDown)
		{
			AddExpectedErrorDuringTearDown();
		}

		TEST_METHOD(BackingFixture_ResetsStateBetweenTestsPartOne)
		{
			ASSERT_THAT(AreEqual(1, SomeNumber));
		}

		TEST_METHOD(BackingFixture_ResetsStateBetweenTestsPartTwo)
		{
			ASSERT_THAT(AreEqual(1, SomeNumber));
		}
	};

	TEST_CLASS(TestFixtureConstructor, "TestFramework.CQTest.Core")
	{
		bool SetupCalled = false;
		uint32 SomeNumber = 0;

		TestFixtureConstructor()
		{
			SetupCalled = true;
			SomeNumber++;
		}

	protected:
		TEST_METHOD(ConstructorIsCalled_BeforeRunTest)
		{
			ASSERT_THAT(IsTrue(SetupCalled));
		}

		TEST_METHOD(BackingFixture_ResetsStateBetweenTestsPartOne)
		{
			ASSERT_THAT(AreEqual(1, SomeNumber));
		}

		TEST_METHOD(BackingFixture_ResetsStateBetweenTestsPartTwo)
		{
			ASSERT_THAT(AreEqual(1, SomeNumber));
		}
	};


	TEST_CLASS(TestAssertionInBefore, "TestFramework.CQTest.Core")
	{
		FString ExpectedError = TEXT("Expected Error Message");

		BEFORE_EACH() 
		{
			Assert.Fail(ExpectedError);
		}

		AFTER_EACH() 
		{
			ClearExpectedError(*this->TestRunner, ExpectedError);
		}

		TEST_METHOD(BeforeTest_AssertionFailure_DoesNotRunTestMethod)
		{
			Assert.Fail(TEXT("TEST_METHOD should not run if assertion fails in BEFORE_EACH"));
		}
	};

	TEST_CLASS(BeforeAndAfterAll, "TestFramework.CQTest.Core")
	{
		inline static uint32 BeforeAllCallCount = 0;
		FString ExpectedError = TEXT("Expected Error Message");

		BEFORE_ALL()
		{
			BeforeAllCallCount++;
		}

		AFTER_ALL()
		{
			BeforeAllCallCount = 0;
		}

		BEFORE_EACH()
		{
			ASSERT_THAT(AreEqual(1, BeforeAllCallCount));
		}

		AFTER_EACH()
		{
			ClearExpectedError(*this->TestRunner, ExpectedError);
			ASSERT_THAT(AreEqual(1, BeforeAllCallCount));
		}

		TEST_METHOD(StaticMember_IsAvailable_DuringTest)
		{
			ASSERT_THAT(AreEqual(1, BeforeAllCallCount));
		}

		TEST_METHOD(BeforeAll_IsCalled_OnlyOnce)
		{
			ASSERT_THAT(AreEqual(1, BeforeAllCallCount));
		}

		TEST_METHOD(AfterAll_WhenTestFails_StillFires)
		{
			Assert.Fail(ExpectedError);
		}
	};

	static_assert(HasBeforeAll<BeforeAndAfterAll>);
	static_assert(HasAfterAll<BeforeAndAfterAll>);

	TEST(ValidateAfterAll, "TestFramework.CQTest.Core")
	{
		ASSERT_THAT(AreEqual(0, BeforeAndAfterAll::BeforeAllCallCount));
	}

	// --------------------------------------------------------
	// Latent commands are awaited
	// --------------------------------------------------------
	template <typename Test>
	class FMinimumCallCommand : public IAutomationLatentCommand
	{
	public:
		FMinimumCallCommand(Test* InTest, int32 expectedCount)
			: ExecutingTest(InTest), ExpectedCount(expectedCount) {}

		bool Update() override
		{
			if (ExecutingTest)
			{
				ExecutingTest->IncrementExecutedCommandsCount();
			}
			CurrentCount++;
			return CurrentCount == ExpectedCount;
		}

		Test* ExecutingTest;
		int32 ExpectedCount{ 0 };
		int32 CurrentCount{ 0 };
	};

	TEST_CLASS(AddCommandTests, "TestFramework.CQTest.Core")
	{
		void SetExpectedExecutedCommandsCount(int32 count)
		{
			ExpectedExecutedCommandsCount = count;
		}
		void IncrementExecutedCommandsCount()
		{
			ExecutedCommandsCount++;
		}

		int32 ExpectedExecutedCommandsCount{ 0 };
		int32 ExecutedCommandsCount{ 0 };

		BEFORE_EACH()
		{
			for (int32 i = 0; i < 3; i++)
			{
				AddCommand(new FMinimumCallCommand(this, i + 1));
			}
		}

		AFTER_EACH()
		{
			if (ExpectedExecutedCommandsCount > 0)
			{
				ASSERT_THAT(AreEqual(ExpectedExecutedCommandsCount, ExecutedCommandsCount));
			}
		}

		TEST_METHOD(Test_WithCommandsInBeforeTest_ExecutesCommandsBeforeRun)
		{
			ASSERT_THAT(AreEqual(1 + 2 + 3, ExecutedCommandsCount));
		}

		TEST_METHOD(Test_WithLatentCommandsInTest_ExecutesCommandsBeforeTearDown)
		{
			ExpectedExecutedCommandsCount = ExecutedCommandsCount * 2;
			for (int32 i = 0; i < 3; i++)
			{
				AddCommand(new FMinimumCallCommand(this, i + 1));
			}
		}
	};

	TEST_CLASS(LatentActionsTest, "TestFramework.CQTest.Core")
	{
		TArray<FString> CommandLog;

		TFunction<bool(LatentActionsTest*)> Assertion;
		TArray<FString> KnownStrings = {
			TEXT("One"),
			TEXT("Two"),
			TEXT("Three"),
			TEXT("Four")
		};

		AFTER_EACH()
		{
			ASSERT_THAT(IsTrue(Assertion(this)));
		}

		TEST_METHOD(Do_OnCommandBuilder_AddsLatentCommand)
		{
			TestCommandBuilder.Do([&]() { CommandLog.Add(KnownStrings[0]); });
			ASSERT_THAT(IsTrue(CommandLog.IsEmpty()));
			Assertion = [](LatentActionsTest* test) {
				return !test->CommandLog.IsEmpty();
			};
		}

		TEST_METHOD(MultipleDoCalls_OnCommandBuilder_AddsAllCommands)
		{
			TestCommandBuilder.Do([&]() {
				CommandLog.Add(KnownStrings[0]);
			})
			.Do([&]() {
				CommandLog.Add(KnownStrings[1]);
			})
			.Do([&]() {
				CommandLog.Add(KnownStrings[2]);
			});

			Assertion = [](LatentActionsTest* test) {
				if (test->CommandLog.Num() != 3)
				{
					return false;
				}
				for (int32 Index = 0; Index < 3; Index++)
				{
					if (test->CommandLog[Index] != test->KnownStrings[Index])
					{
						return false;
					}
				}
				return true;
			};
		}

		TEST_METHOD(DoAndAddCommand_InTheSameTest_AddCommandsInOrder)
		{
			TestCommandBuilder.Do([&]() { CommandLog.Add(KnownStrings[0]); });
			AddCommand(new FExecute(*TestRunner, [&]() { CommandLog.Add(KnownStrings[1]); }));
			TestCommandBuilder.Do([&]() { CommandLog.Add(KnownStrings[2]); });
			AddCommand(new FExecute(*TestRunner, [&]() { CommandLog.Add(KnownStrings[3]); }));

			Assertion = [](LatentActionsTest* test) {
				if (test->CommandLog.Num() != 4)
				{
					return false;
				}
				for (int32 Index = 0; Index < 4; Index++)
				{
					if (test->CommandLog[Index] != test->KnownStrings[Index])
					{
						return false;
					}
				}
				return true;
			};
		}
	};

	TEST_CLASS(LatentActionOnTearDown, "TestFramework.CQTest.Core")
	{
		FString ExpectedError = TEXT("ExpectedErrorShouldBeCleared");

		TEST_METHOD(Teardown_Fires_AfterAll)
		{
			TestCommandBuilder.Do([&]() {
				Assert.Fail(ExpectedError);
			})
			.OnTearDown([&]() {
				ClearExpectedError(*TestRunner, ExpectedError);
			});
		}

		TEST_METHOD(Teardown_Fires_InReverseOrder)
		{
			TestCommandBuilder.OnTearDown([&]() {
				ClearExpectedError(*TestRunner, ExpectedError);
			}).OnTearDown([&]() {
				Assert.Fail(ExpectedError);
			});
		}
	};


	TEST_CLASS(LatentActionErrors, "TestFramework.CQTest.Core")
	{
		FString ExpectedError = TEXT("ExpectedError");

		AFTER_EACH()
		{
			ClearExpectedError(*TestRunner, ExpectedError);
		}

		TEST_METHOD(Assertion_InLatentActions_PreventsAdditionalLatentActions)
		{
			TestCommandBuilder.Do([&]() {
				Assert.Fail(ExpectedError);
			})
			.Then([&]() {
				Assert.Fail("Unexpected Error");
			});
		}

		TEST_METHOD(Timeout_InLatentActions_ProvidesTimeoutInErrorMessage)
		{
			Assert.ExpectErrorRegex(TEXT("\\d{2,} milliseconds"));
			TestCommandBuilder.StartWhen([]() {
				return false;
				}, FTimespan::FromMilliseconds(50));
		}
	};

	// --------------------------------------------------------
	// Tickable Game Objects Tick
	// --------------------------------------------------------
	struct FTestTickable : public FTickableGameObject
	{
		virtual TStatId GetStatId() const override
		{
			return TStatId();
		}

		virtual void Tick(float DeltaTime) override
		{
			TickCount++;
		}

		virtual bool IsTickableInEditor() const override
		{
			return true;
		}

		void ResetTickCount()
		{
			TickCount = 0;
		}

		uint32 TickCount = 0;
	};

	TEST_CLASS(GameObjectsTickTest, "TestFramework.CQTest.Core")
	{
		BEFORE_EACH()
		{
			Tickable.ResetTickCount();
			AddCommand(new FWaitUntil(*TestRunner, [&]() { return Tickable.TickCount > 2; }));
		}
		AFTER_EACH()
		{
			ASSERT_THAT(IsTrue(Tickable.TickCount > 2));
		}

		FTestTickable Tickable{};

		TEST_METHOD(TestWithTickableGameObject_WaitingForTicksInSetup_WillAllowGameObjectToTick)
		{
			TestCommandBuilder.Do([this]() { ASSERT_THAT(IsTrue(Tickable.TickCount > 2)); });
		}

		TEST_METHOD(TestWithTickableGameObject_WaitingForTicksInSetup_WillBeCompleteDuringRunStep)
		{
			ASSERT_THAT(IsTrue(Tickable.TickCount > 2));
		}
	};

	DEFINE_LOG_CATEGORY_STATIC(TestSuppressLog, Log, All);

	TEST_CLASS(SuppressWarningsAndErrors, "TestFramework.CQTest.Core")
	{
		bool bExpectError{ false };
		bool bExpectWarning{ false };
		FString ExpectedError = TEXT("ExpectedError");

		AFTER_EACH()
		{
			static FString FormattedMessage = FString::Printf(TEXT("TestSuppressLog: %s"), *ExpectedError);
			if (bExpectError)
			{
				ClearExpectedErrors(*this->TestRunner, { TEXT("will be marked as failing due to errors"), FormattedMessage });
			}
			if (bExpectWarning)
			{
				ClearExpectedWarning(*this->TestRunner, FormattedMessage);
			}
		}

		TEST_METHOD(SuppressWarnings_WithWarnings_SuppressesWarning)
		{
			TestRunner->SetSuppressLogWarnings();
			UE_LOGF(TestSuppressLog, Warning, "This log should be suppressed");
		}

		TEST_METHOD(SuppressWarnings_WithError_DoesNotSuppressError)
		{
			TestRunner->SetSuppressLogWarnings();
			bExpectError = true;
			UE_LOGF(TestSuppressLog, Error, "%ls", *ExpectedError);
		}

		TEST_METHOD(SuppressErrors_WithWarnings_DoesNotSuppressWarning)
		{
			TestRunner->SetSuppressLogErrors();
			bExpectWarning = true;
			UE_LOGF(TestSuppressLog, Warning, "%ls", *ExpectedError);
		}

		TEST_METHOD(SuppressErrors_WithError_SuppressesError)
		{
			TestRunner->SetSuppressLogErrors();
			UE_LOGF(TestSuppressLog, Error, "This log should be suppressed");
		}

		TEST_METHOD(SuppressErrors_ChangedMidTest_RespectsChanges)
		{
			TestRunner->SetSuppressLogErrors();
			UE_LOGF(TestSuppressLog, Error, "This log should be suppressed");
			TestRunner->SetSuppressLogErrors(ECQTestSuppressLogBehavior::False);
			bExpectError = true;
			UE_LOGF(TestSuppressLog, Error, "%ls", *ExpectedError);
		}
	};

	TEST_CLASS(AutomationEvents, "TestFramework.CQTest.Core")
	{
		bool bIsExpectingError{ false };
		bool bIsExpectingWarning{ false };
		FString ExpectedMessage = TEXT("ExpectedMessage");

		AFTER_EACH()
		{
			if (bIsExpectingError)
			{
				ClearExpectedErrors(*this->TestRunner, { TEXT("will be marked as failing due to errors"), ExpectedMessage });
			}
			if (bIsExpectingWarning)
			{
				ClearExpectedWarning(*this->TestRunner, ExpectedMessage);
			}
		}

		TEST_METHOD(InfoEvent)
		{
			AddInfo(ExpectedMessage);
			ASSERT_THAT(IsTrue(DoesEventExist(*this->TestRunner, FAutomationEvent(EAutomationEventType::Info, ExpectedMessage))));
		}

		TEST_METHOD(WarningEvent)
		{
			bIsExpectingWarning = true;
			AddWarning(ExpectedMessage);
			ASSERT_THAT(IsTrue(DoesEventExist(*this->TestRunner, FAutomationEvent(EAutomationEventType::Warning, ExpectedMessage))));
		}

		TEST_METHOD(ConditionalEvent)
		{
			FString InvalidErrorMessage = TEXT("This should not be an error");
			AddErrorIfFalse(true, InvalidErrorMessage);
			ASSERT_THAT(IsFalse(DoesEventExist(*this->TestRunner, FAutomationEvent(EAutomationEventType::Error, InvalidErrorMessage))));
			AddErrorIfFalse(false, ExpectedMessage);
			bIsExpectingError = true;
			ASSERT_THAT(IsTrue(DoesEventExist(*this->TestRunner, FAutomationEvent(EAutomationEventType::Error, ExpectedMessage))));
		}

		TEST_METHOD(ErrorEvent)
		{
			bIsExpectingError = true;
			AddError(ExpectedMessage);
			ASSERT_THAT(IsTrue(DoesEventExist(*this->TestRunner, FAutomationEvent(EAutomationEventType::Error, ExpectedMessage))));
		}
	};

	TEST_CLASS_WITH_TAGS(CQTags, "TestFramework.CQTest.Core.Tags", "[CQTest][ExtraTag][CQTaggedTests]")
	{
		TEST_METHOD(CheckTagsAreValid)
		{
			const FString TestTags = TestRunner->GetTestTags();
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQTest]"))));
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[ExtraTag]"))));
		}

		TEST_METHOD_WITH_TAGS(CheckTagsAreValid_HasExpectedMethodTags, "[CQClassWithTagsMethod]")
		{
			const FString ThisMethodName = TEXT("CheckTagsAreValid_HasExpectedMethodTags");
			const FString TestTags = TestRunner->GetTestMethodTags(ThisMethodName);
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQTest]"))));
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[ExtraTag]"))));
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQClassWithTagsMethod]"))));
		}

		TEST_METHOD_WITH_TAGS(CheckTagsAreValid_HasExpectedMethodTagsNoLeak, "[CQClassWithTagsNoLeakMethod]")
		{
			const FString ThisMethodName = TEXT("CheckTagsAreValid_HasExpectedMethodTagsNoLeak");
			const FString TestTags = TestRunner->GetTestMethodTags(ThisMethodName);
			ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQClassWithTagsNoLeakMethod]"))));
			ASSERT_THAT(IsFalse(TestTags.Contains(TEXT("[CQClassWithTagsMethod]"))));
		}
	};
} // namespace CQTestTests
