// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTestUnitTestHelper.h"

bool DoesEventExist(FAutomationTestBase& TestRunner, const FAutomationEvent& Event)
{
	FAutomationTestExecutionInfo TestInfo;
	TestRunner.GetExecutionInfo(TestInfo);
	const TArray<FAutomationExecutionEntry>& TestEntries = TestInfo.GetEntries();
	if (TestEntries.Num() > 0)
	{
		for (const FAutomationExecutionEntry& Entry : TestEntries)
		{
			if (Entry.Event.Type == Event.Type && Entry.Event.Message.Equals(Event.Message))
			{
				return true;
			}
		}
	}

	return false;
}

void ClearExpectedErrors(FAutomationTestBase& TestRunner, const TArray<FString>& ExpectedErrors)
{
	FAutomationTestExecutionInfo testInfo;
	TestRunner.GetExecutionInfo(testInfo);
	if (testInfo.GetErrorTotal() == 0)
	{
		return;
	}
	for(auto & ExpectedError : ExpectedErrors)
	{
		testInfo.RemoveAllEvents([&ExpectedError](FAutomationEvent& event) {
			return event.Message.Contains(ExpectedError);
			});
	}
	if (testInfo.GetErrorTotal() == 0)
	{
		TestRunner.ClearExecutionInfo();
	}
}

void ClearExpectedError(FAutomationTestBase& TestRunner, const FString& ExpectedError)
{
	ClearExpectedErrors(TestRunner, { ExpectedError });
}

void ClearExpectedWarnings(FAutomationTestBase& TestRunner, const TArray<FString>& ExpectedWarnings)
{
	FAutomationTestExecutionInfo testInfo;
	TestRunner.GetExecutionInfo(testInfo);
	if (testInfo.GetWarningTotal() == 0)
	{
		return;
	}
	for (auto& ExpectedWarning : ExpectedWarnings)
	{
		testInfo.RemoveAllEvents([&ExpectedWarning](FAutomationEvent& event) {
			return event.Message.Contains(ExpectedWarning);
			});
	}
	if (testInfo.GetWarningTotal() == 0)
	{
		TestRunner.ClearExecutionInfo();
	}
}

void ClearExpectedWarning(FAutomationTestBase& TestRunner, const FString& ExpectedWarning)
{
	ClearExpectedWarnings(TestRunner, { ExpectedWarning });
}