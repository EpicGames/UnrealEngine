// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/AutomationTest.h"

bool DoesEventExist(FAutomationTestBase& TestRunner, const FAutomationEvent& Event);

// When testing the framework, occasionally we need the error to live on the test framework
// for short-circuiting logic.  We use this method to manually clear the error
// (Normally from AFTER_EACH) to cause the test to pass
void ClearExpectedErrors(FAutomationTestBase& TestRunner, const TArray<FString>& ExpectedErrors);
void ClearExpectedError(FAutomationTestBase& TestRunner, const FString& ExpectedError);
void ClearExpectedWarnings(FAutomationTestBase& TestRunner, const TArray<FString>& ExpectedWarnings);
void ClearExpectedWarning(FAutomationTestBase& TestRunner, const FString& ExpectedWarning);
