// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "CQTestUnitTestHelper.h"

TEST_CLASS(CommandBuilderTests, "TestFramework.CQTest.Core")
{
	FTestCommandBuilder CommandBuilder{*TestRunner};

	TEST_METHOD(Do_ThenBuild_IncludesCommand)
	{
		bool invoked = false;
		auto command = CommandBuilder.Do([&invoked]() { invoked = true; }).Build();

		ASSERT_THAT(IsTrue(command->Update()));
		ASSERT_THAT(IsTrue(invoked));
	}

	TEST_METHOD(Build_WithoutCommands_ReturnsNullptr)
	{
		auto command = CommandBuilder.Build();
		ASSERT_THAT(IsNull(command));
	}

	TEST_METHOD(StartWhen_CreatesWaitUntilCommand)
	{
		bool done = false;
		auto command = CommandBuilder.StartWhen([&done]() { return done; }).Build();

		ASSERT_THAT(IsFalse(command->Update()));
		done = true;
		ASSERT_THAT(IsTrue(command->Update()));
	}

	TEST_METHOD(WaitDelay_WaitsUntilDurationElapsed)
	{
		bool done = false;
		FTimespan Duration = FTimespan::FromMilliseconds(200);
		FDateTime EndTime = FDateTime::UtcNow() + Duration;
		auto command = CommandBuilder
			.WaitDelay(Duration)
			.Then([this, &EndTime, &done]()
			{
				ASSERT_THAT(IsTrue(FDateTime::UtcNow() >= EndTime));
				done = true;
			}).Build();
		
		while (!done)
		{
			command->Update();
		}
		ASSERT_THAT(IsTrue(done));
	}

	TEST_METHOD(WaitDelay_InterruptOnError)
	{
		const FString ExpectedError = TEXT("Error reported outside WaitDelay");

		FTimespan Duration = FTimespan::FromSeconds(10);
		FDateTime EndTime = FDateTime::UtcNow() + Duration;
		auto command = CommandBuilder
			.WaitDelay(Duration).Build();

		ASSERT_THAT(IsFalse(command->Update()));
		AddError(ExpectedError);
		ASSERT_THAT(IsTrue(command->Update()));
		ASSERT_THAT(IsTrue(FDateTime::UtcNow() < EndTime));

		ClearExpectedError(*this->TestRunner, ExpectedError);
	}

	TEST_METHOD(Build_AfterBuild_ReturnsNullptr)
	{
		auto command = CommandBuilder.Do([]() {}).Build();
		auto secondTime = CommandBuilder.Build();

		ASSERT_THAT(IsNotNull(command));
		ASSERT_THAT(IsNull(secondTime));
	}
	
	TEST_METHOD(DoAsync_ThenBuild_IncludesCommand)
	{
		TPromise<bool> Promise;
		bool bAsyncActionInvoked = false;

		auto Command = CommandBuilder.DoAsync<bool>(
			[&]()
			{
				bAsyncActionInvoked = true;
				return TAsyncResult<bool>(Promise.GetFuture(), nullptr, nullptr);
			}
		).Build();
		
		// Start async action
		ASSERT_THAT(IsFalse(bAsyncActionInvoked));
		ASSERT_THAT(IsFalse(Command->Update()));
		ASSERT_THAT(IsTrue(bAsyncActionInvoked));
		
		// Wait for async result
		ASSERT_THAT(IsFalse(Command->Update()));
		Promise.SetValue(true);
		ASSERT_THAT(IsTrue(Command->Update()));
	}
	
	TEST_METHOD(DoAsync_WithResultCallback_ThenBuild_IncludesCommand)
	{
		TPromise<bool> Promise;
		bool bAsyncActionInvoked = false;
		bool bResultCallbackInvoked = false;

		auto Command = CommandBuilder.DoAsync<bool>(
			[&]()
			{
				bAsyncActionInvoked = true;
				return TAsyncResult<bool>(Promise.GetFuture(), nullptr, nullptr); 
			},
			[&](bool)
			{
				bResultCallbackInvoked = true; 
			}
		).Build();
		
		// Start async action
		ASSERT_THAT(IsFalse(bAsyncActionInvoked));
		ASSERT_THAT(IsFalse(Command->Update()));
		ASSERT_THAT(IsTrue(bAsyncActionInvoked));
		
		// Wait for async result
		ASSERT_THAT(IsFalse(Command->Update()));
		Promise.SetValue(true);
		ASSERT_THAT(IsFalse(Command->Update()));
		
		// Handle result
		ASSERT_THAT(IsFalse(bResultCallbackInvoked));
		ASSERT_THAT(IsTrue(Command->Update()));
		ASSERT_THAT(IsTrue(bResultCallbackInvoked));
	}

	TEST_METHOD(DoAsync_WaitsAsyncResultForSpecifiedDuration)
	{
		TPromise<bool> Promise;
		const FTimespan AsyncActionTimeout = FTimespan::FromMilliseconds(200);
		const FDateTime StartTime = FDateTime::UtcNow();
		const FDateTime MaxEndTime = StartTime + FTimespan::FromMilliseconds(500);

		auto Command = CommandBuilder.DoAsync<bool>(
			[&]() { return TAsyncResult<bool>(Promise.GetFuture(), nullptr, nullptr); },
			AsyncActionTimeout
		).Build();

		bool bDone = false;
		while (!bDone && FDateTime::UtcNow() < MaxEndTime)
		{
			bDone = Command->Update();
		}

		const FDateTime EndTime = FDateTime::UtcNow();
		ClearExpectedError(*TestRunner, "Latent command timed out");
		// Set value to destroy the promise correctly
		Promise.SetValue(true);

		ASSERT_THAT(IsTrue(bDone));
		ASSERT_THAT(IsTrue(EndTime >= StartTime + AsyncActionTimeout));
	}

	TEST_METHOD(DoAsync_ShouldProcessResultOfType_Void)
	{
		TPromise<void> Promise;

		auto Command = CommandBuilder.DoAsync<void>(
			[&]()
			{
				return TAsyncResult<void>(Promise.GetFuture(), nullptr, nullptr);
			}
		).Build();
		
		// Start async action
		ASSERT_THAT(IsFalse(Command->Update()));
		
		// Wait for async result
		Promise.SetValue();
		ASSERT_THAT(IsTrue(Command->Update()));
	}

	TEST_METHOD(DoAsync_ShouldProcessResultOfType_Class)
	{
		class FTestContainer
		{
		public:
			FTestContainer(int32 InValue) : Value(InValue) {}
			int32 Value;
		};

		TPromise<FTestContainer> Promise;
		FTestContainer Result(0);

		auto Command = CommandBuilder.DoAsync<FTestContainer>(
			[&]()
			{
				return TAsyncResult<FTestContainer>(Promise.GetFuture(), nullptr, nullptr);
			},
			[&](const FTestContainer& InResult)
			{
				Result = InResult;
			}
		).Build();
		
		// Start async action
		ASSERT_THAT(IsFalse(Command->Update()));
		
		const FTestContainer ExpectedResult(567);
		Promise.SetValue(ExpectedResult);
		// Need two updates: the first to verify that the async result is ready, the second to invoke the callback.
		bool bDone = false;
		for (int32 i = 0; !bDone && i < 2; i++)
		{
			bDone = Command->Update();
		}
		ASSERT_THAT(IsTrue(bDone));
		ASSERT_THAT(AreEqual(ExpectedResult.Value, Result.Value));
	}

	TEST_METHOD(DoAsync_ShouldProcessResultOfType_Reference)
	{
		TPromise<int32&> Promise;
		int32 Value = 12345;
		const int32 ExpectedValue = 6789;

		auto Command = CommandBuilder.DoAsync<int32&>(
			[&]()
			{
				return TAsyncResult<int32&>(Promise.GetFuture(), nullptr, nullptr);
			},
			[&](int32& InResult)
			{
				InResult = ExpectedValue;
			}
		).Build();
		
		// Start async action
		ASSERT_THAT(IsFalse(Command->Update()));

		Promise.SetValue(Value);
		// Need two updates: the first to verify that the async result is ready, the second to invoke the callback.
		bool bDone = false;
		for (int32 i = 0; !bDone && i < 2; i++)
		{
			bDone = Command->Update();
		}
		ASSERT_THAT(IsTrue(bDone));
		ASSERT_THAT(AreEqual(ExpectedValue, Value));
	}

	TEST_METHOD(UntilAsync_ThenBuild_IncludesCommand)
	{
		TPromise<bool> Promise;
		bool bAsyncActionInvoked = false;
		bool bConditionChecked = false;
		bool bConditionResult = false;

		auto Command = CommandBuilder.UntilAsync<bool>(
			[&]()
			{
				bAsyncActionInvoked = true;
				return TAsyncResult<bool>(Promise.GetFuture(), nullptr, nullptr);
			},
			[&](bool)
			{
				bConditionChecked = true;
				return bConditionResult;
			}
		).Build();
		
		// Start async action
		ASSERT_THAT(IsFalse(bAsyncActionInvoked));
		ASSERT_THAT(IsFalse(Command->Update()));
		ASSERT_THAT(IsTrue(bAsyncActionInvoked));
		
		// Wait for async result
		ASSERT_THAT(IsFalse(Command->Update()));
		Promise.SetValue(true);
		ASSERT_THAT(IsFalse(Command->Update()));
		
		// Start checking condition
		ASSERT_THAT(IsFalse(bConditionChecked));
		ASSERT_THAT(IsFalse(Command->Update()));
		ASSERT_THAT(IsTrue(bConditionChecked));
		
		// Stop execution when the condition is met
		bConditionResult = true;
		bConditionChecked = false;
		ASSERT_THAT(IsTrue(Command->Update()));
		ASSERT_THAT(IsTrue(bConditionChecked));
	}

	TEST_METHOD(UntilAsync_WaitsAsyncResultForSpecifiedDuration)
	{
		TPromise<bool> Promise;
		const FTimespan AsyncActionTimeout = FTimespan::FromMilliseconds(200);
		const FTimespan ConditionTimeout = FTimespan::FromSeconds(10);

		auto Command = CommandBuilder.UntilAsync<bool>(
			[&]() { return TAsyncResult<bool>(Promise.GetFuture(), nullptr, nullptr); },
			[](bool) { return false; },
			AsyncActionTimeout,
			ConditionTimeout
		).Build();

		bool bDone = false;
		const FDateTime StartTime = FDateTime::UtcNow();
		const FDateTime MaxEndTime = StartTime + FTimespan::FromMilliseconds(5000);
		while (!bDone && FDateTime::UtcNow() < MaxEndTime)
		{
			bDone = Command->Update();
		}

		const FDateTime EndTime = FDateTime::UtcNow();
		ClearExpectedError(*TestRunner, "Latent command timed out");
		// Set value to destroy the promise correctly
		Promise.SetValue(true);
		
		ASSERT_THAT(IsTrue(bDone));
		ASSERT_THAT(IsTrue(EndTime >= StartTime + AsyncActionTimeout));
	}

	TEST_METHOD(UntilAsync_WaitsConditionForSpecifiedDuration)
	{
		TPromise<bool> Promise;
		const FTimespan AsyncActionTimeout = FTimespan::FromSeconds(10);
		const FTimespan ConditionTimeout = FTimespan::FromMilliseconds(200);

		auto Command = CommandBuilder.UntilAsync<bool>(
			[&]() { return TAsyncResult<bool>(Promise.GetFuture(), nullptr, nullptr); },
			[](bool) { return false; },
			AsyncActionTimeout,
			ConditionTimeout
		).Build();
		
		Promise.SetValue(true);

		bool bDone = false;
		const FDateTime StartTime = FDateTime::UtcNow();
		const FDateTime MaxEndTime = StartTime + FTimespan::FromMilliseconds(5000);
		while (!bDone && FDateTime::UtcNow() < MaxEndTime)
		{
			bDone = Command->Update();
		}

		const FDateTime EndTime = FDateTime::UtcNow();
		ClearExpectedError(*TestRunner, "Latent command timed out");
		
		ASSERT_THAT(IsTrue(bDone));
		ASSERT_THAT(IsTrue(EndTime >= StartTime + ConditionTimeout));
	}

	TEST_METHOD(UntilAsync_ShouldProcessResultOfType_Class)
	{
		class FTestContainer
		{
		public:
			FTestContainer(int32 InValue) : Value(InValue) {}
			int32 Value;
		};

		TPromise<FTestContainer> Promise;
		FTestContainer Result(0);

		auto Command = CommandBuilder.UntilAsync<FTestContainer>(
			[&]()
			{
				return TAsyncResult<FTestContainer>(Promise.GetFuture(), nullptr, nullptr);
			},
			[&](const FTestContainer& InResult)
			{
				Result = InResult;
				return true;
			}
		).Build();
		
		// Start async action
		ASSERT_THAT(IsFalse(Command->Update()));
		
		const FTestContainer ExpectedResult(567);
		Promise.SetValue(ExpectedResult);
		// Need two updates: the first to verify that the async result is ready, the second to invoke the callback.
		bool bDone = false;
		for (int32 i = 0; !bDone && i < 2; i++)
		{
			bDone = Command->Update();
		}
		ASSERT_THAT(IsTrue(bDone));
		ASSERT_THAT(AreEqual(ExpectedResult.Value, Result.Value));
	}

	TEST_METHOD(UntilAsync_ShouldProcessResultOfType_Reference)
	{
		TPromise<int32&> Promise;
		int32 Value = 12345;
		const int32 ExpectedValue = 6789;

		auto Command = CommandBuilder.UntilAsync<int32&>(
			[&]()
			{
				return TAsyncResult<int32&>(Promise.GetFuture(), nullptr, nullptr);
			},
			[&](int32& Result)
			{
				Result = ExpectedValue;
				return true;
			}
		).Build();
		
		// Start async action
		ASSERT_THAT(IsFalse(Command->Update()));

		Promise.SetValue(Value);
		// Need two updates: the first to verify that the async result is ready, the second to invoke the callback.
		bool bDone = false;
		for (int32 i = 0; !bDone && i < 2; i++)
		{
			bDone = Command->Update();
		}
		ASSERT_THAT(IsTrue(bDone));
		ASSERT_THAT(AreEqual(ExpectedValue, Value));
	}
	
	TEST_METHOD(ThenAsync_ThenBuild_IncludesCommand)
	{
		TPromise<bool> Promise;
		bool bAsyncActionInvoked = false;

		auto Command = CommandBuilder.ThenAsync<bool>(
			[&]()
			{
				bAsyncActionInvoked = true;
				return TAsyncResult<bool>(Promise.GetFuture(), nullptr, nullptr);
			}
		).Build();
		
		// Start async action
		ASSERT_THAT(IsFalse(bAsyncActionInvoked));
		ASSERT_THAT(IsFalse(Command->Update()));
		ASSERT_THAT(IsTrue(bAsyncActionInvoked));
		
		// Wait for async result
		ASSERT_THAT(IsFalse(Command->Update()));
		Promise.SetValue(true);
		ASSERT_THAT(IsTrue(Command->Update()));
	}
	
	TEST_METHOD(ThenAsync_WithResultCallback_ThenBuild_IncludesCommand)
	{
		TPromise<bool> Promise;
		bool bAsyncActionInvoked = false;
		bool bResultCallbackInvoked = false;

		auto Command = CommandBuilder.ThenAsync<bool>(
			[&]()
			{
				bAsyncActionInvoked = true;
				return TAsyncResult<bool>(Promise.GetFuture(), nullptr, nullptr); 
			},
			[&](bool)
			{
				bResultCallbackInvoked = true; 
			}
		).Build();
		
		// Start async action
		ASSERT_THAT(IsFalse(bAsyncActionInvoked));
		ASSERT_THAT(IsFalse(Command->Update()));
		ASSERT_THAT(IsTrue(bAsyncActionInvoked));
		
		// Wait for async result
		ASSERT_THAT(IsFalse(Command->Update()));
		Promise.SetValue(true);
		ASSERT_THAT(IsFalse(Command->Update()));
		
		// Handle result
		ASSERT_THAT(IsFalse(bResultCallbackInvoked));
		ASSERT_THAT(IsTrue(Command->Update()));
		ASSERT_THAT(IsTrue(bResultCallbackInvoked));
	}

	TEST_METHOD(ThenAsync_WaitsAsyncResultForSpecifiedDuration)
	{
		TPromise<bool> Promise;
		const FTimespan AsyncActionTimeout = FTimespan::FromMilliseconds(200);
		const FDateTime StartTime = FDateTime::UtcNow();
		const FDateTime MaxEndTime = StartTime + FTimespan::FromMilliseconds(500);

		auto Command = CommandBuilder.ThenAsync<bool>(
			[&]() { return TAsyncResult<bool>(Promise.GetFuture(), nullptr, nullptr); },
			AsyncActionTimeout
		).Build();

		bool bDone = false;
		while (!bDone && FDateTime::UtcNow() < MaxEndTime)
		{
			bDone = Command->Update();
		}

		const FDateTime EndTime = FDateTime::UtcNow();
		ClearExpectedError(*TestRunner, "Latent command timed out");
		// Set value to destroy the promise correctly
		Promise.SetValue(true);

		ASSERT_THAT(IsTrue(bDone));
		ASSERT_THAT(IsTrue(EndTime >= StartTime + AsyncActionTimeout));
	}
};

/* These tests illustrate different approaches to running functions that return TAsyncResult within Command Builder */
TEST_CLASS(CommandBuilderForAsyncResultTests, "TestFramework.CQTest.Core")
{
	class FFakeAsyncTask
	{
	public:
		FFakeAsyncTask(FAutomationTestBase& InTestRunner, int InValue, int32 InDuration) :
			TestRunner(InTestRunner),
			Value(InValue),
			Duration(InDuration)
		{
			check(Duration > 0);
		}

		~FFakeAsyncTask()
		{
			if (IsRunning) { Promise->SetValue({}); }
		}

		TAsyncResult<int> ProduceValue()
		{
			if (TestRunner.AddErrorIfFalse(!IsRunning, TEXT("Async task has already been started")))
			{
				IsRunning = true;
				Promise = MakeShared<TPromise<int>>();
				return TAsyncResult<int>(Promise->GetFuture(), nullptr, nullptr);
			}
			return TAsyncResult<int>();
		}

		void Update()
		{
			if (!IsRunning) { return; }

			if (--Duration > 0) { return; }

			Promise->SetValue(Value);
			IsRunning = false;
		}

	private:
		FAutomationTestBase& TestRunner;
		TSharedPtr<TPromise<int>> Promise;
		int Value;
		int32 Duration;
		bool IsRunning = false;
	};

	class FFakeBackgroundTask
	{
	public:
		FFakeBackgroundTask(FAutomationTestBase& InTestRunner, int InValue, int32 InDuration) :
			TestRunner(InTestRunner),
			Value(InValue),
			Duration(InDuration) {}

		bool IsReady(int InValue)
		{
			if (TestRunner.AddErrorIfFalse(Value == InValue,
				FString::Printf(TEXT("Incorrect value. Expected: %d, actual: %d"), Value, InValue)))
			{
				return Duration == 0;
			}
			return false;
		}

		void Update() { if (Duration > 0) { Duration--; } }

	private:
		FAutomationTestBase& TestRunner;
		const int Value;
		int32 Duration;
	};

	FTestCommandBuilder CommandBuilder{ *TestRunner };

	/* Execute an async task without checking the return value, using a sequence of general-purpose commands. */
	TEST_METHOD(Execute_StepByStep)
	{
		const int32 TaskDurationInTicks = 5;
		FFakeAsyncTask AsyncTask(*TestRunner, 0, TaskDurationInTicks);

		TAsyncResult<int> AsyncResult;

		auto Command = CommandBuilder
			.Do(TEXT("Start producing value"), [&]() { AsyncResult = AsyncTask.ProduceValue(); })
			.Until(TEXT("Value produced"), [&]() { return AsyncResult.GetFuture().IsReady(); })
			.Build();

		bool bDone = false;
		int32 Timeout = TaskDurationInTicks + 1;
		while (!bDone && Timeout--)
		{
			bDone = Command->Update();
			AsyncTask.Update();
		}

		ASSERT_THAT(IsTrue(bDone));
	}
	
	/* Execute an async task without checking the return value, using a DoAsync command. */
	TEST_METHOD(Execute_ByDoAsync)
	{
		const int32 TaskDurationInTicks = 5;
		FFakeAsyncTask Task(*TestRunner, 0, TaskDurationInTicks);

		auto Command = CommandBuilder
			.DoAsync<int>(TEXT("Produce value"), [&]() { return Task.ProduceValue(); })
			.Build();

		bool bDone = false;
		int32 Timeout = TaskDurationInTicks + 1;
		while (!bDone && Timeout--)
		{
			bDone = Command->Update();
			Task.Update();
		}

		ASSERT_THAT(IsTrue(bDone));
	}
	
	/* Execute an async task and retrieve the return value using a sequence of general-purpose commands. */
	TEST_METHOD(ExecuteAndGetResult_StepByStep)
	{
		const int ExpectedValue = 123;
		const int32 TaskDurationInTicks = 5;
		FFakeAsyncTask AsyncTask(*TestRunner, ExpectedValue, TaskDurationInTicks);

		TAsyncResult<int> AsyncResult;
		int Result = 0;

		auto Command = CommandBuilder
			.Do(TEXT("Start producing value"), [&]() { AsyncResult = AsyncTask.ProduceValue(); })
			.Until(TEXT("Value produced"), [&]() { return AsyncResult.GetFuture().IsReady(); })
			.Then(TEXT("Save value"), [&]() { Result = AsyncResult.GetFuture().Get(); })
			.Build();

		bool bDone = false;
		int32 Timeout = TaskDurationInTicks + 2;
		while (!bDone && Timeout--)
		{
			bDone = Command->Update();
			AsyncTask.Update();
		}

		ASSERT_THAT(AreEqual(ExpectedValue, Result));
	}
	
	/* Execute an async task and retrieve the return value using a DoAsync command. */
	TEST_METHOD(ExecuteAndGetResult_ByDoAsync)
	{
		const int ExpectedValue = 456;
		const int32 TaskDurationInTicks = 5;
		FFakeAsyncTask Task(*TestRunner, ExpectedValue, TaskDurationInTicks);

		int Result = 0;

		auto Command = CommandBuilder
			.DoAsync<int>(TEXT("Produce value"),
				[&]() { return Task.ProduceValue(); },
				[&](int InResult) { Result = InResult; }
			)
			.Build();

		bool bDone = false;
		int32 Timeout = TaskDurationInTicks + 2;
		while (!bDone && Timeout--)
		{
			bDone = Command->Update();
			Task.Update();
		}

		ASSERT_THAT(AreEqual(ExpectedValue, Result));
	}
	
	/* Execute an async task and wait for the condition specified by the return value, using a sequence of general-purpose commands. */
	TEST_METHOD(ExecuteAndWait_StepByStep)
	{
		const int ProducedValue = 789;
		const int32 AsyncTaskDuration = 5;
		const int32 BackgroundTaskDuration = 10;

		FFakeAsyncTask AsyncTask(*TestRunner, ProducedValue, AsyncTaskDuration);
		FFakeBackgroundTask BackgroundTask(*TestRunner, ProducedValue, BackgroundTaskDuration);

		TAsyncResult<int> AsyncResult;

		auto Command = CommandBuilder
			.Do(TEXT("Start producing value"), [&]() { AsyncResult = AsyncTask.ProduceValue(); })
			.Until(TEXT("Value produced"), [&]() { return AsyncResult.GetFuture().IsReady(); })
			.Until(TEXT("Resource is ready"), [&]() { return BackgroundTask.IsReady(AsyncResult.GetFuture().Get()); })
			.Build();

		bool bDone = false;
		int32 Timeout = BackgroundTaskDuration + 1;
		while (!bDone && Timeout--)
		{
			bDone = Command->Update();
			AsyncTask.Update();
			BackgroundTask.Update();
		}

		ASSERT_THAT(IsTrue(bDone));
		ASSERT_THAT(IsTrue(BackgroundTask.IsReady(ProducedValue)));
	}
	
	/* Execute an async task and wait for the condition specified by the return value, using an UntilAsync command. */
	TEST_METHOD(ExecuteAndWait_ByUntilAsync)
	{
		const int ProducedValue = 987;
		const int32 AsyncTaskDuration = 5;
		const int32 BackgroundTaskDuration = 10;

		FFakeAsyncTask AsyncTask(*TestRunner, ProducedValue, AsyncTaskDuration);
		FFakeBackgroundTask BackgroundTask(*TestRunner, ProducedValue, BackgroundTaskDuration);

		auto Command = CommandBuilder
			.UntilAsync<int>(TEXT("Produced resource is ready"),
				[&]() { return AsyncTask.ProduceValue(); },
				[&](int InResult) { return BackgroundTask.IsReady(InResult); }
			)
			.Build();

		bool bDone = false;
		int32 Timeout = BackgroundTaskDuration + 1;
		while (!bDone && Timeout--)
		{
			bDone = Command->Update();
			AsyncTask.Update();
			BackgroundTask.Update();
		}

		ASSERT_THAT(IsTrue(bDone));
		ASSERT_THAT(IsTrue(BackgroundTask.IsReady(ProducedValue)));
	}
};