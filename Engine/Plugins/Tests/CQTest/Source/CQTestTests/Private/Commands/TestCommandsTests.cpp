// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "CQTestUnitTestHelper.h"

#include "CQTestSettings.h"

TEST_CLASS(RunSequenceBasicTests, "TestFramework.CQTest.Core")
{
	TSharedPtr<IAutomationLatentCommand> Cmd1;
	TSharedPtr<IAutomationLatentCommand> Cmd2;
	FString Cmd1Name = "One";
	FString Cmd2Name = "Two";

	TArray<FString> Log;

	BEFORE_EACH() {
		Cmd1 = MakeShared<FExecute>(*TestRunner, [&]() { Log.Add(Cmd1Name); }, *Cmd1Name);
		Cmd2 = MakeShared<FExecute>(*TestRunner, [&]() { Log.Add(Cmd2Name); }, *Cmd2Name);
	}

	TEST_METHOD(Update_WithRemainingCommands_ReturnsFalse)
	{
		FRunSequence sequence(Cmd1, Cmd2);
		ASSERT_THAT(IsFalse(sequence.Update()));
	}

	TEST_METHOD(Update_OnLastCommand_ReturnTrue)
	{
		FRunSequence sequence(Cmd1);
		ASSERT_THAT(IsTrue(sequence.Update()));
	}

	TEST_METHOD(Append_ANewCommand_AddsCommandToEnd)
	{
		FRunSequence sequence{ Cmd1 };
		sequence.Append(Cmd2);

		sequence.Update();
		sequence.Update();

		ASSERT_THAT(AreEqual(2, Log.Num()));
		ASSERT_THAT(AreEqual(Cmd1Name, Log[0]));
		ASSERT_THAT(AreEqual(Cmd2Name, Log[1]));
	}

	TEST_METHOD(Prepend_ANewCommand_AddsCommandToBeginning)
	{
		FRunSequence sequence{ Cmd1 };
		sequence.Prepend(Cmd2);

		sequence.Update();
		sequence.Update();

		ASSERT_THAT(AreEqual(2, Log.Num()));
		ASSERT_THAT(AreEqual(Cmd2Name, Log[0]));
		ASSERT_THAT(AreEqual(Cmd1Name, Log[1]));
	}
};

struct FCommandLog
{
	TArray<FString> Commands;
};

class FNamedCommand : public IAutomationLatentCommand
{
public:
	FNamedCommand(TArray<FString>& CommandLog, FString Name)
		: Log(CommandLog), CommandName(Name) {}

	bool Update() override
	{
		Log.Add(CommandName);
		return true;
	}

	TArray<FString>& Log;
	FString CommandName;
};

class FTickingNamedCommand : public FNamedCommand
{
public:
	FTickingNamedCommand(TArray<FString>& CommandLog, FString Name, int32 Ticks)
		: FNamedCommand(CommandLog, Name), ExpectedCount(Ticks) {}

	bool Update() override
	{
		if (CurrentCount == ExpectedCount)
		{
			return true;
		}

		Log.Add(CommandName);
		CurrentCount++;
		return false;
	}

	int32 ExpectedCount{ 0 };
	int32 CurrentCount{ 0 };
};


TEST_CLASS(RunSequenceTests, "TestFramework.CQTest.Core")
{
	const TArray<FString> Names =
	{
		"Zero",
		"One",
		"Two",
		"Three",
		"Four",
		"Five",
		"Six",
		"Seven",
		"Eight"
	};

	TFunction<bool(RunSequenceTests*)> Assertion;
	TArray<FString> CommandLog;

	AFTER_EACH()
	{
		ASSERT_THAT(IsTrue(Assertion(this)));
	}

	TEST_METHOD(RunSequence_WithZeroCommands_DoesNotFail)
	{
		AddCommand(new FRunSequence());
		Assertion = [](RunSequenceTests* test) {
			return test->CommandLog.IsEmpty();
		};
	}

	TEST_METHOD(RunSequence_WithOneCommand_RunsCommand)
	{
		AddCommand(new FRunSequence(MakeShared<FNamedCommand>(CommandLog, Names[0])));
		Assertion = [](RunSequenceTests* test) {
			return test->CommandLog.Num() == 1 && test->CommandLog[0] == test->Names[0];
		};
	}

	TEST_METHOD(RunSequence_WithNamedCommands_RunsCommandsInOrder)
	{
		TArray<TSharedPtr<FNamedCommand>> Commands;
		Commands.Add(MakeShared<FNamedCommand>(CommandLog, Names[0]));
		Commands.Add(MakeShared<FNamedCommand>(CommandLog, Names[1]));
		Commands.Add(MakeShared<FNamedCommand>(CommandLog, Names[2]));
		Commands.Add(MakeShared<FNamedCommand>(CommandLog, Names[3]));
		Commands.Add(MakeShared<FNamedCommand>(CommandLog, Names[4]));

		AddCommand(new FRunSequence(Commands));

		Assertion = [](RunSequenceTests* test) {
			if (test->CommandLog.Num() != 5)
			{
				return false;
			}

			for (int32 i = 0; i < 5; i++)
			{
				if (test->CommandLog[i] != test->Names[i])
				{
					return false;
				}
			}

			return true;
		};
	}

	TEST_METHOD(RunSequence_WithTickingCommands_RunsCommandsInOrder)
	{
		TArray<TSharedPtr<FTickingNamedCommand>> Commands;
		Commands.Add(MakeShared<FTickingNamedCommand>(CommandLog, Names[0], 3));
		Commands.Add(MakeShared<FTickingNamedCommand>(CommandLog, Names[1], 3));
		Commands.Add(MakeShared<FTickingNamedCommand>(CommandLog, Names[2], 3));
		Commands.Add(MakeShared<FTickingNamedCommand>(CommandLog, Names[3], 3));
		Commands.Add(MakeShared<FTickingNamedCommand>(CommandLog, Names[4], 3));

		AddCommand(new FRunSequence(Commands));

		Assertion = [](RunSequenceTests* test) {
			if (test->CommandLog.Num() != 15)
			{
				return false;
			}

			int32 NameIndex = -1;
			for (int32 CommandIndex = 0; CommandIndex < test->CommandLog.Num(); CommandIndex++)
			{
				if (CommandIndex % 3 == 0)
				{
					NameIndex++;
				}

				if (test->CommandLog[CommandIndex] != test->Names[NameIndex])
				{
					return false;
				}
			}

			return true;
		};
	}

	TEST_METHOD(RunSequence_WithSequences_RunsCommandsInOrder)
	{
		TArray<TSharedPtr<FNamedCommand>> Cmds1;
		TArray<TSharedPtr<FNamedCommand>> Cmds2;
		TArray<TSharedPtr<FNamedCommand>> Cmds3;

		Cmds1.Add(MakeShared<FNamedCommand>(CommandLog, Names[0]));
		Cmds1.Add(MakeShared<FNamedCommand>(CommandLog, Names[1]));
		Cmds1.Add(MakeShared<FNamedCommand>(CommandLog, Names[2]));

		Cmds2.Add(MakeShared<FNamedCommand>(CommandLog, Names[3]));
		Cmds2.Add(MakeShared<FNamedCommand>(CommandLog, Names[4]));
		Cmds2.Add(MakeShared<FNamedCommand>(CommandLog, Names[5]));

		Cmds3.Add(MakeShared<FNamedCommand>(CommandLog, Names[6]));
		Cmds3.Add(MakeShared<FNamedCommand>(CommandLog, Names[7]));
		Cmds3.Add(MakeShared<FNamedCommand>(CommandLog, Names[8]));

		AddCommand(new FRunSequence(MakeShared<FRunSequence>(Cmds1), MakeShared<FRunSequence>(Cmds2), MakeShared<FRunSequence>(Cmds3)));

		Assertion = [](RunSequenceTests* test) {
			for (int32 i = 0; i < 9; i++)
			{
				if (test->CommandLog[i] != test->Names[i])
				{
					return false;
				}
			}

			return true;
		};
	}

	TEST_METHOD(RunSequence_WithSeparateSequences_RunsCommandsInOrder)
	{
		TArray<TSharedPtr<FNamedCommand>> Cmds1;
		TArray<TSharedPtr<FNamedCommand>> Cmds2;
		TArray<TSharedPtr<FNamedCommand>> Cmds3;

		Cmds1.Add(MakeShared<FNamedCommand>(CommandLog, Names[0]));
		Cmds1.Add(MakeShared<FNamedCommand>(CommandLog, Names[1]));
		Cmds1.Add(MakeShared<FNamedCommand>(CommandLog, Names[2]));

		Cmds2.Add(MakeShared<FNamedCommand>(CommandLog, Names[3]));
		Cmds2.Add(MakeShared<FNamedCommand>(CommandLog, Names[4]));
		Cmds2.Add(MakeShared<FNamedCommand>(CommandLog, Names[5]));

		Cmds3.Add(MakeShared<FNamedCommand>(CommandLog, Names[6]));
		Cmds3.Add(MakeShared<FNamedCommand>(CommandLog, Names[7]));
		Cmds3.Add(MakeShared<FNamedCommand>(CommandLog, Names[8]));

		AddCommand(new FRunSequence(Cmds1));
		AddCommand(new FRunSequence(Cmds2));
		AddCommand(new FRunSequence(Cmds3));

		Assertion = [](RunSequenceTests* test) {
			for (int32 i = 0; i < 9; i++)
			{
				if (test->CommandLog[i] != test->Names[i])
				{
					return false;
				}
			}

			return true;
		};
	}

	TEST_METHOD(RunSequence_WithUntilCommands_RunsCommandsInOrder)
	{
		TArray<TSharedPtr<IAutomationLatentCommand>> Cmds;
		Cmds.Add(MakeShared<FWaitUntil>(*TestRunner, [&]() {
			static int32 attempt = 0;
			CommandLog.Add(Names[0]);
			if (++attempt > 3)
			{
				attempt = 0;
				return true;
			}
			return false;
			}));
		Cmds.Add(MakeShared<FWaitUntil>(*TestRunner, [&]() {
			static int32 attempt = 0;
			CommandLog.Add(Names[1]);
			if (++attempt > 4)
			{
				attempt = 0;
				return true;
			}
			return false;
			}));

		AddCommand(new FRunSequence(Cmds));

		Assertion = [](RunSequenceTests* test) {
			return test->CommandLog.Num() == 9;
		};
	}
};

class FFakeAsyncTask
{
public:
	FFakeAsyncTask(FAutomationTestBase& InTestRunner) : TestRunner{ InTestRunner } {}

	~FFakeAsyncTask()
	{
		if (InProgress()) { Complete(0); }
	}

	TAsyncResult<int> Start()
	{
		if (TestRunner.AddErrorIfFalse(!IsRunning, TEXT("Async task has already been started")))
		{
			IsRunning = true;
			Promise = MakeShared<TPromise<int>>();
			return TAsyncResult<int>(Promise->GetFuture(), nullptr, nullptr);
		}
		return TAsyncResult<int>();
	}

	void Complete(int Value)
	{
		if (TestRunner.AddErrorIfFalse(IsRunning, TEXT("Cannot set the async task result if it's not running")))
		{
			IsRunning = false;
			Promise->SetValue(Value);
		}
	}

	bool InProgress() const { return IsRunning; }

private:
	bool IsRunning = false;
	FAutomationTestBase& TestRunner;
	TSharedPtr<TPromise<int>> Promise;
};

TEST_CLASS(AsyncExecuteBasicTests, "TestFramework.CQTest.Core")
{
	TEST_METHOD(AsyncExecute_InvokesAsyncActionWhenUpdated)
	{
		FFakeAsyncTask Task(*TestRunner);

		TAsyncExecute<int> AsyncExecute = TAsyncExecute<int>(
			*TestRunner,
			[&]() { return Task.Start(); }
		);

		ASSERT_THAT(IsFalse(Task.InProgress(), TEXT("Default state of async task is invalid")));
		ASSERT_THAT(IsFalse(AsyncExecute.Update(), TEXT("Command stopped execution early")));
		ASSERT_THAT(IsTrue(Task.InProgress(), TEXT("Async task hasn't been started")));
	}

	TEST_METHOD(AsyncExecute_InvokesAsyncActionOnce)
	{
		FFakeAsyncTask Task(*TestRunner);
		int32 Counter = 0;

		TAsyncExecute<int> AsyncExecute = TAsyncExecute<int>(
			*TestRunner,
			[&]()
			{
				Counter++;
				return Task.Start();
			}
		);
		
		int32 Ticks = 3;
		bool bDone = false;
		while (!bDone && Ticks--)
		{
			bDone = AsyncExecute.Update();
		}
		ASSERT_THAT(IsFalse(bDone, TEXT("Command stopped execution early")));
		ASSERT_THAT(AreEqual(1, Counter, TEXT("Async action was invoked multiple times")));
	}

	TEST_METHOD(AsyncExecute_WaitsUntilAsyncResultIsReady)
	{
		FFakeAsyncTask Task(*TestRunner);

		TAsyncExecute<int> AsyncExecute = TAsyncExecute<int>(
			*TestRunner,
			[&]() { return Task.Start(); }
		);

		// Start async task
		ASSERT_THAT(IsFalse(AsyncExecute.Update(), TEXT("Command stopped execution early")));

		bool bDone = false;
		for (int32 i = 0; !bDone && i < 10; i++)
		{
			bDone = AsyncExecute.Update();
		}
		ASSERT_THAT(IsFalse(bDone, TEXT("Command stopped execution before the async task was completed")));

		Task.Complete(0);
		ASSERT_THAT(IsTrue(AsyncExecute.Update(), TEXT("Command failed to stop execution after the async task was completed")));
	}

	TEST_METHOD(AsyncExecute_FExecute_CompletesWhenAsyncActionIsCompleted)
	{
		FFakeAsyncTask Task(*TestRunner);

		TAsyncExecute<int, FExecute> AsyncExecute = TAsyncExecute<int, FExecute>(
			*TestRunner,
			[&]() { return Task.Start(); },
			FTimespan::FromSeconds(1),
			nullptr,
			[](int) {}
		);

		// Start async task
		ASSERT_THAT(IsFalse(AsyncExecute.Update(), TEXT("Command stopped execution early")));

		Task.Complete(0);
		// Need two updates: the first to verify that the async result is ready, the second to invoke the callback.
		bool bDone = false;
		for (int32 i = 0; !bDone && i < 2; i++)
		{
			bDone = AsyncExecute.Update();
		}
		ASSERT_THAT(IsTrue(bDone, TEXT("Command failed to stop execution after the async task was completed")));
	}

	TEST_METHOD(AsyncExecute_FExecute_InvokesResultCallbackWhenAsyncActionIsCompleted)
	{
		FFakeAsyncTask Task(*TestRunner);
		bool bInvoked = false;

		TAsyncExecute<int, FExecute> AsyncExecute = TAsyncExecute<int, FExecute>(
			*TestRunner,
			[&]() { return Task.Start(); },
			FTimespan::FromSeconds(1),
			nullptr,
			[&](int) { bInvoked = true; }
		);

		// Start async task
		ASSERT_THAT(IsFalse(AsyncExecute.Update(), TEXT("Command stopped execution early")));

		ASSERT_THAT(IsFalse(bInvoked, TEXT("Callback was invoked early")));
		Task.Complete(0);

		// Need two updates: the first to verify that the async result is ready, the second to invoke the callback.
		bool bDone = false;
		for (int32 i = 0; !bDone && i < 2; i++)
		{
			bDone = AsyncExecute.Update();
		}
		ASSERT_THAT(IsTrue(bDone, TEXT("Command did not stop execution")));
		ASSERT_THAT(IsTrue(bInvoked, TEXT("Callback hasn't been invoked")));
	}

	TEST_METHOD(AsyncExecute_FExecute_InvokesResultCallbackOnce)
	{
		FFakeAsyncTask Task(*TestRunner);
		int Counter = 0;

		TAsyncExecute<int, FExecute> AsyncExecute = TAsyncExecute<int, FExecute>(
			*TestRunner,
			[&]() { return Task.Start(); },
			FTimespan::FromSeconds(1),
			nullptr,
			[&](int) { Counter++; }
		);

		// Start async task
		ASSERT_THAT(IsFalse(AsyncExecute.Update(), TEXT("Command stopped execution early")));

		Task.Complete(0);
		// Need two updates: the first to verify that the async result is ready, the second to invoke the callback.
		bool bDone = false;
		for (int32 i = 0; !bDone && i < 2; i++)
		{
			bDone = AsyncExecute.Update();
		}
		ASSERT_THAT(IsTrue(bDone, TEXT("Command did not stop execution")));
		ASSERT_THAT(AreEqual(1, Counter, "Callback was invoked multiple times"));
	}

	TEST_METHOD(AsyncExecute_FExecute_PassesValueToResultCallback)
	{
		FFakeAsyncTask Task(*TestRunner);
		int Result = 0;

		TAsyncExecute<int, FExecute> AsyncExecute = TAsyncExecute<int, FExecute>(
			*TestRunner,
			[&]() { return Task.Start(); },
			FTimespan::FromSeconds(1),
			nullptr,
			[&](int InResult) { Result = InResult; }
		);

		// Start async task
		ASSERT_THAT(IsFalse(AsyncExecute.Update(), TEXT("Command stopped execution early")));

		const int ExpectedResult = 5;
		Task.Complete(ExpectedResult);
		// Need two updates: the first to verify that the async result is ready, the second to invoke the callback.
		bool bDone = false;
		for (int32 i = 0; !bDone && i < 2; i++)
		{
			bDone = AsyncExecute.Update();
		}
		ASSERT_THAT(IsTrue(bDone, TEXT("Command did not stop execution")));
		ASSERT_THAT(AreEqual(ExpectedResult, Result, TEXT("Incorrect value passed to result callback")));
	}

	TEST_METHOD(AsyncExecute_FWaitUntil_InvokesResultCallbackOnEveryUpdateWhenAsyncActionIsCompleted)
	{
		FFakeAsyncTask Task(*TestRunner);
		int Counter = 0;

		TAsyncExecute<int, FWaitUntil> AsyncExecute = TAsyncExecute<int, FWaitUntil>(
			*TestRunner,
			[&]() { return Task.Start(); },
			FTimespan::FromSeconds(1),
			nullptr,
			[&](int)
			{
				Counter++;
				return false;
			}
		);

		// Start async task
		ASSERT_THAT(IsFalse(AsyncExecute.Update(), TEXT("Command stopped execution early")));

		ASSERT_THAT(AreEqual(0, Counter, TEXT("Callback was invoked early")));

		Task.Complete(0);
		const int32 Ticks = 10;
		bool bDone = false;
		for (int32 i = 0; !bDone && i < Ticks; i++)
		{
			bDone = AsyncExecute.Update();
		}
		ASSERT_THAT(IsFalse(bDone, TEXT("Command execution stopped before the condition was met")));
		// One tick will be used to verify that the async result is ready
		ASSERT_THAT(AreEqual(Ticks - 1, Counter, TEXT("Incorrect number of callback invocations")));
	}

	TEST_METHOD(AsyncExecute_FWaitUntil_PassesValueToResultCallback)
	{
		FFakeAsyncTask Task(*TestRunner);
		const int ExpectedResult = 6;
		bool bAllValuesAreValid = true;
		bool bInvoked = false;

		TAsyncExecute<int, FWaitUntil> AsyncExecute = TAsyncExecute<int, FWaitUntil>(
			*TestRunner,
			[&]() { return Task.Start(); },
			FTimespan::FromSeconds(1),
			nullptr,
			[&](int Value)
			{
				bInvoked = true;
				if (Value != ExpectedResult)
				{
					bAllValuesAreValid = false;
				}
				return false;
			}
		);

		// Start async task
		ASSERT_THAT(IsFalse(AsyncExecute.Update(), TEXT("Command stopped execution early")));

		Task.Complete(ExpectedResult);
		bool bDone = false;
		for (int32 i = 0; !bDone && i < 10; i++)
		{
			bDone = AsyncExecute.Update();
		}
		ASSERT_THAT(IsTrue(bInvoked, TEXT("Result callback was not invoked")));
		ASSERT_THAT(IsTrue(bAllValuesAreValid, TEXT("Incorrect value passed to result callback")));
	}

	TEST_METHOD(AsyncExecute_FWaitUntil_CompletesWhenResultCallbackReturnsTrue)
	{
		FFakeAsyncTask Task(*TestRunner);
		bool bCallbackReturnValue = false;

		TAsyncExecute<int, FWaitUntil> AsyncExecute = TAsyncExecute<int, FWaitUntil>(
			*TestRunner,
			[&]() { return Task.Start(); },
			FTimespan::FromSeconds(1),
			nullptr,
			[&](int) { return bCallbackReturnValue; }
		);

		// Start async task
		ASSERT_THAT(IsFalse(AsyncExecute.Update(), TEXT("Command stopped execution early")));

		Task.Complete(0);
		bool bDone = false;
		for (int i = 0; !bDone && i < 10; i++)
		{
			bDone = AsyncExecute.Update();
		}
		ASSERT_THAT(IsFalse(bDone, TEXT("Command stopped execution before the condition was met")));

		bCallbackReturnValue = true;
		ASSERT_THAT(IsTrue(AsyncExecute.Update(), TEXT("Command failed to stop execution after the condition was met")));
	}
};

TEST_CLASS(AsyncExecuteTimeoutTests, "TestFramework.CQTest.Core")
{
	TSharedPtr<FFakeAsyncTask> Task;

	BEFORE_EACH()
	{
		Task = MakeShared<FFakeAsyncTask>(*TestRunner);
	}

	AFTER_EACH()
	{
		ASSERT_THAT(IsTrue(Task->InProgress(), TEXT("Async task hasn't been started")));
		ClearExpectedError(*TestRunner, "Latent command timed out");
		Task.Reset();
	}

	TEST_METHOD(AsyncExecute_FExecute_DoesNotInvokeResultCallbackOnTimeout)
	{
		AddCommand(MakeShared<TAsyncExecute<int, FExecute>>(
			*TestRunner,
			[this]() { return Task->Start(); },
			FTimespan::FromMilliseconds(1),
			nullptr,
			[this](int)
			{
				AddError("Result callback should not be invoked after timeout");
			}
		));
	}

	TEST_METHOD(AsyncExecute_FWaitUntil_DoesNotInvokeResultCallbackOnTimeout)
	{
		AddCommand(MakeShared<TAsyncExecute<int, FWaitUntil>>(
			*TestRunner,
			[this]() { return Task->Start(); },
			FTimespan::FromMilliseconds(1),
			nullptr,
			[this](int)
			{
				AddError("Result callback should not be invoked after timeout");
				return true;
			}
		));
	}
};

TEST_CLASS(WaitUntilTests, "TestFramework.CQTest.Core")
{
	TEST_METHOD(Timeout_WithNoValueProvided_IsGreaterThanZero)
	{
		FWaitUntil command(*TestRunner, []() { return true; });
		ASSERT_THAT(IsTrue(command.Timeout > FTimespan::Zero()));
	}

	TEST_METHOD(Timeout_WithSpecificValue_IsUsed)
	{
		FTimespan Timeout = FTimespan::FromSeconds(100);
		FWaitUntil command(*TestRunner, []() { return true; }, Timeout);
		ASSERT_THAT(AreEqual(Timeout, command.Timeout));
	}

	TEST_METHOD(Timeout_WithDefaultValue_UsesCVar)
	{
		FWaitUntil command(*TestRunner, []() { return true; }, CQTest::DefaultTimeout);
		ASSERT_THAT(IsNear(CQTestConsoleVariables::CommandTimeout, static_cast<float>(command.Timeout.GetTotalSeconds()), SMALL_NUMBER));
	}

	TEST_METHOD(Timeout_WithOverriddenCvar_UsesOverriddenValue)
	{
		IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(CQTestConsoleVariables::CommandTimeoutName);
		ASSERT_THAT(IsNotNull(ConsoleVariable));

		const float NewTimeout = ConsoleVariable->GetFloat() + 1;
		FString NewTimeoutStr = FString::Printf(TEXT("%f"), NewTimeout);
		TSharedPtr<FScopedTestEnvironment> TestEnvironment = UCQTestSettings::SetTestClassTimeouts(FTimespan::FromSeconds(NewTimeout));

		FWaitUntil command(*TestRunner, []() { return true; }, CQTest::DefaultTimeout);
		ASSERT_THAT(IsNear(NewTimeout, static_cast<float>(command.Timeout.GetTotalSeconds()), SMALL_NUMBER));
	}
};

TEST_CLASS(WaitUntilClassTimeoutTest, "TestFramework.CQTest.Core")
{
	TEST_METHOD(Timeout_SetInBeforeEach_PersistsInTest)
	{
		IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(CQTestConsoleVariables::CommandTimeoutName);
		ASSERT_THAT(IsNotNull(ConsoleVariable));

		const double TimeoutValue = ConsoleVariable->GetFloat() + 1;
		TSharedPtr<FScopedTestEnvironment> TestEnvironment = UCQTestSettings::SetTestClassTimeouts(FTimespan::FromSeconds(TimeoutValue));

		FWaitUntil command(*TestRunner, []() { return true; }, CQTest::DefaultTimeout);
		ASSERT_THAT(IsNear(TimeoutValue, command.Timeout.GetTotalSeconds(), 0.01));
	}
};

#if WITH_EDITOR
// Test checks to make sure that editing the timeout property is reflected. Can only be done within the Editor.
TEST_CLASS_WITH_FLAGS(WaitUntilUserSettingTimeout, "TestFramework.CQTest.Core", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	double ExpectedTimeout{0.0};
	FString OriginalTimeout;

	bool SetPropertyValue(const FName& PropertyName, const FString& PropertyValue, FString& OutError)
	{
		UCQTestSettings* DefaultSettings = GetMutableDefault<UCQTestSettings>();
		if (!IsValid(DefaultSettings))
		{
			OutError = TEXT("Could not load default CQTest Settings.");
			return false;
		}

		if (FProperty* Property = FindFProperty<FFloatProperty>(UCQTestSettings::StaticClass(), PropertyName))
		{
			Property->ImportText_InContainer(*PropertyValue, DefaultSettings, DefaultSettings, PPF_None);

			FPropertyChangedEvent ChangeEvent(Property, EPropertyChangeType::ValueSet);
			DefaultSettings->PostEditChangeProperty(ChangeEvent);
			return true;
		}

		OutError = FString::Format(TEXT("Property '{0}' was not found in the CQTest Settings."), { PropertyName.ToString() });
		return false;
	}

	BEFORE_EACH()
	{
		float DefaultTimeout = GetDefault<UCQTestSettings>()->CommandTimeout;
		ExpectedTimeout = DefaultTimeout + 1;
		OriginalTimeout = FString::SanitizeFloat(DefaultTimeout);

		FString ErrorMessage;
		const bool bWasUpdated = SetPropertyValue(TEXT("CommandTimeout"), FString::SanitizeFloat(ExpectedTimeout), ErrorMessage);
		AddErrorIfFalse(bWasUpdated, ErrorMessage);
	}

	AFTER_EACH()
	{
		FString ErrorMessage;
		const bool bWasUpdated = SetPropertyValue(TEXT("CommandTimeout"), OriginalTimeout, ErrorMessage);
		AddErrorIfFalse(bWasUpdated, ErrorMessage);
	}

	TEST_METHOD(Timeout_SetInBeforeEach_PersistsInTest)
	{
		FWaitUntil command(*TestRunner, []() { return true; }, CQTest::DefaultTimeout);
		ASSERT_THAT(IsNear(ExpectedTimeout, command.Timeout.GetTotalSeconds(), 0.01));
	}
};
#endif // WITH_EDITOR