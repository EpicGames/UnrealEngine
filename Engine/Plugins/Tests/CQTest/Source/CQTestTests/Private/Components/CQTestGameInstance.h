// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/GameInstance.h"

#include "CQTestGameInstance.generated.h"

/** Basic GameInstance with a test value. */
UCLASS()
class UCQGameInstanceClass : public UGameInstance
{
	GENERATED_BODY()

public:
	int TestValue{ 42 };
};
