// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/GameModeBase.h"
#include "CQTestGameMode.generated.h"

/** Basic GameMode with a test value. */
UCLASS()
class ACQTestGameMode : public AGameModeBase
{
	GENERATED_BODY()

public:
	int32 TestValue{ 42 };
};