// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"

#include "Components/CQTestSlateComponent.h"
#include "Framework/Application/SlateApplication.h"

TEST_CLASS(SlateTests, "TestFramework.CQTest.UI")
{
	TUniquePtr<FCQTestSlateComponent> SlateComponent{ nullptr };

	BEFORE_EACH()
	{
		SlateComponent = MakeUnique<FCQTestSlateComponent>();
	}

	TEST_METHOD(HaveTicksElapsed_WithoutTicking_ReturnsFalse)
	{
		ASSERT_THAT(IsTrue(SlateComponent->HaveTicksElapsed(0)));
	}

	TEST_METHOD(HaveTicksElapsed_AfterTicking_ReturnsTrue)
	{
		ASSERT_THAT(IsFalse(SlateComponent->HaveTicksElapsed(1)));
		FSlateApplication::Get().Tick();
		ASSERT_THAT(IsTrue(SlateComponent->HaveTicksElapsed(1)));
	}

	TEST_METHOD(HaveTicksElapsed_InUntilCommand_EventuallyReturnsTrue)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]() { ASSERT_THAT(IsTrue(SlateComponent->GetTotalElapsedTicks() >= 3)); });
	}

	TEST_METHOD(ElapsedTicks_WhenReused_AwaitsAllTheTicks)
	{
		TestCommandBuilder
			.StartWhen([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Until([this]() { return SlateComponent->HaveTicksElapsed(3); })
			.Then([this]() { ASSERT_THAT(IsTrue(SlateComponent->GetTotalElapsedTicks() >= 6)); });
	}
};