// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "Components/MapTestSpawner.h"
#include "GameFramework/Pawn.h"

#if WITH_EDITOR && WITH_AUTOMATION_TESTS

#include "Editor.h"

TEST_CLASS_WITH_FLAGS(MapSpawnHelperTests, "TestFramework.CQTest.Map", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	TUniquePtr<FMapTestSpawner> Spawner;

	BEFORE_EACH() {
		Spawner = FMapTestSpawner::CreateFromTempLevel(TestCommandBuilder);
		ASSERT_THAT(IsNotNull(Spawner));

		Spawner->AddWaitUntilLoadedCommand(TestRunner);
		TestCommandBuilder.Do([this]() {
			// Because we're creating a level for this test, we will also want to populate the level with a Pawn object that can be then tested against
			Spawner->SpawnActor<APawn>();
		});
	}

	TEST_METHOD(MapSpawner_FindsPlayerSpawn)
	{
		TestCommandBuilder.Do([this]() {
			APawn* Player = Spawner->FindFirstPlayerPawn();
			ASSERT_THAT(IsNotNull(Player));
		});
	}

	TEST_METHOD(MapSpawner_PIEEndsEarly)
	{
		TestCommandBuilder.Do([this]() { GEditor->RequestEndPlayMap(); })
		.Until([this]() { return !GEditor->IsPlaySessionInProgress(); })
		.Until([this]() { return !IsValid(GEditor->PlayWorld); })
		.Then([this]() { ASSERT_THAT(IsNull(GEditor->PlayWorld)); });
	}
};

#endif // WITH_EDITOR && WITH_AUTOMATION_TESTS