// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "Components/PIENetworkComponent.h"
#include "CQTestGameInstance.h"
#include "CQTestGameMode.h"
#include "CQTestUnitTestHelper.h"
#include "TestReplicatedActor.h"
#include "Tests/AutomationCommon.h"

#include "Engine/World.h"

#if ENABLE_PIE_NETWORK_TEST

/**
 * The tests below showcase how the `FBasePIENetworkComponent` can be used to setup and simulate a client/server network using the default replication system.
 * Please enable the CQTestExperimental Engine plugin to see and run tests against the experimental Iris replication system (https://dev.epicgames.com/documentation/en-us/unreal-engine/iris-replication-system-in-unreal-engine).
*/

NETWORK_TEST_CLASS(StateTest, "TestFramework.CQTest.Network")
{
	struct DerivedState : public FBasePIENetworkComponentState
	{
		int32 IndependentNumber = 0;
	};

	int32 SharedNumber = 0;
	FPIENetworkComponent<DerivedState> Network{ TestRunner, TestCommandBuilder, bInitializing };
	
	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("0"));
	}

	BEFORE_EACH()
	{
		FNetworkComponentBuilder<DerivedState>()
			.WithClients(3)
			.WithGameInstanceClass(UGameInstance::StaticClass())
			.WithGameMode(AGameModeBase::StaticClass())
			.Build(Network);
	}
	
	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(Network_WithMultipleSteps_TriggersStepsInOrder)
	{
		Network
			.ThenServer([this](DerivedState&) { ASSERT_THAT(AreEqual(0, SharedNumber++)); })
			.ThenClient(0, [this](DerivedState&) { ASSERT_THAT(AreEqual(1, SharedNumber++)); })
			.ThenServer([this](DerivedState&) { ASSERT_THAT(AreEqual(2, SharedNumber++)); })
			.Then([this]() { ASSERT_THAT(AreEqual(3, SharedNumber++)); });
	}

	TEST_METHOD(Network_WithServerCommands_RetainsStateBetweenCalls)
	{
		Network
			.ThenServer([this](DerivedState& State) { ASSERT_THAT(AreEqual(0, State.IndependentNumber++)); })
			.ThenServer([this](DerivedState& State) { ASSERT_THAT(AreEqual(1, State.IndependentNumber++)); });
	}

	TEST_METHOD(Network_WithClientCommands_RetainsStateBetweenCalls)
	{
		Network
			.ThenClient(0, [this](DerivedState& State) { ASSERT_THAT(AreEqual(0, State.IndependentNumber++)); })
			.ThenClient(0, [this](DerivedState& State) { ASSERT_THAT(AreEqual(1, State.IndependentNumber++)); });
	}

	TEST_METHOD(Network_WithClientAndServerCommands_DoNotShareState)
	{
		Network
			.ThenServer([this](DerivedState& State) { State.IndependentNumber++; })
			.ThenClient(0, [this](DerivedState& State) { ASSERT_THAT(AreEqual(0, State.IndependentNumber)); });
	}

	TEST_METHOD(Network_WithMultipleClients_DoNotShareState)
	{
		Network
			.ThenClients([this](DerivedState& State) { State.IndependentNumber = State.ClientIndex; })
			.ThenClients([this](DerivedState& State) { ASSERT_THAT(AreEqual(State.ClientIndex, State.IndependentNumber)); });
	}

	TEST_METHOD(Network_WithTickingServerCommand_TicksUntilDone)
	{
		Network
			.UntilServer([this](DerivedState& State) { return ++State.IndependentNumber > 4; })
			.ThenServer([this](DerivedState& State) { ASSERT_THAT(AreEqual(State.IndependentNumber, 5)); });
	}

	TEST_METHOD(Network_WithTickingClientCommands_TicksEachCommand)
	{
		Network
			.UntilClients([this](DerivedState& State) { SharedNumber++;  return ++State.IndependentNumber > 4; })
			.Then([this]() { ASSERT_THAT(AreEqual(15, SharedNumber)); });
	}

	TEST_METHOD(Network_WithClients_ClientConnectionsAreFilledIn)
	{
		Network
			.ThenServer([this](DerivedState& State) {
				ASSERT_THAT(AreEqual(State.ClientCount, 3));
				ASSERT_THAT(AreEqual(State.ClientConnections.Num(), 3));
				for (int32 ClientIndex = 0; ClientIndex < State.ClientConnections.Num(); ++ ClientIndex)
				{
					ASSERT_THAT(IsNotNull(State.ClientConnections[ClientIndex]));
				}
			});
	}
};

NETWORK_TEST_CLASS(ReplicationTest, "TestFramework.CQTest.Network")
{
	struct DerivedState : public FBasePIENetworkComponentState
	{
		ATestReplicatedActor* ReplicatedActor = nullptr;
	};

	const int32 ExpectedReplicatedValue = 42;
	FPIENetworkComponent<DerivedState> Network{ TestRunner, TestCommandBuilder, bInitializing };
	
	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("0"));
	}

	BEFORE_EACH() 
	{
		FNetworkComponentBuilder<DerivedState>()
			.WithGameInstanceClass(UGameInstance::StaticClass())
			.WithGameMode(AGameModeBase::StaticClass())
			.Build(Network);
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(SpawnAndReplicateActor_WithReplicatedActor_ProvidesActorToClients)
	{
		Network.SpawnAndReplicate<ATestReplicatedActor, &DerivedState::ReplicatedActor>()
			.ThenServer([this](DerivedState& ServerState) {
				ASSERT_THAT(IsNotNull(ServerState.ReplicatedActor));
			})
			.ThenClients([this](DerivedState& ClientState) {
				ASSERT_THAT(IsNotNull(ClientState.ReplicatedActor));
			});
	}

	TEST_METHOD(SpawnAndReplicateActor_ThenUpdateProperty_UpdatesPropertyOnClients)
	{
		Network.SpawnAndReplicate<ATestReplicatedActor, &DerivedState::ReplicatedActor>()
			.ThenServer(TEXT("Server Set Value"), [this](DerivedState& ServerState) {
				ServerState.ReplicatedActor->ReplicatedInt = ExpectedReplicatedValue;
			})
			.UntilClients(TEXT("Clients Check Value"), [this](DerivedState& ClientState) {
				return ClientState.ReplicatedActor->ReplicatedInt == ExpectedReplicatedValue;
			});
	}

	TEST_METHOD(SpawnAndReplicateActor_WithSpawnParameters_PassesParametersToSpawnedObject)
	{
		FActorSpawnParameters SpawnParameters;
		SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		Network.SpawnAndReplicate<ATestReplicatedActor, &DerivedState::ReplicatedActor>(SpawnParameters)
			.UntilClients([SpawnParameters](DerivedState& ClientState) {
				return SpawnParameters.SpawnCollisionHandlingOverride == ClientState.ReplicatedActor->SpawnCollisionHandlingMethod;
			});
	}

	TEST_METHOD(SpawnAndReplicateActor_WithBeforeReplicates_InvokesBeforeReplicate) {
		auto BeforeReplicate = [this](ATestReplicatedActor& Actor) {
			Actor.ReplicatedInt = ExpectedReplicatedValue;
		};

		Network.SpawnAndReplicate<ATestReplicatedActor, &DerivedState::ReplicatedActor>(BeforeReplicate)
			.UntilClients([this](DerivedState& ClientState) {
				return ClientState.ReplicatedActor->ReplicatedInt == ExpectedReplicatedValue;
			});
	}

	TEST_METHOD(SpawnAndReplicateActor_WithSpawnParametersAndBeforeReplicate_UsesBoth)
	{
		auto BeforeReplicate = [this](ATestReplicatedActor& Actor) {
			Actor.ReplicatedInt = ExpectedReplicatedValue;
		};

		FActorSpawnParameters SpawnParameters;
		SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		Network.SpawnAndReplicate<ATestReplicatedActor, &DerivedState::ReplicatedActor>(SpawnParameters, BeforeReplicate)
			.UntilClients([this, SpawnParameters](DerivedState& ClientState) {
				ATestReplicatedActor& actor = *ClientState.ReplicatedActor;
				return SpawnParameters.SpawnCollisionHandlingOverride == actor.SpawnCollisionHandlingMethod && actor.ReplicatedInt == ExpectedReplicatedValue;
			});
	}
};

NETWORK_TEST_CLASS(MultipleActorStateReplication, "TestFramework.CQTest.Network")
{
	struct DerivedState : public FBasePIENetworkComponentState
	{
		ATestReplicatedActor* ReplicatedActor1 = nullptr;
		ATestReplicatedActor* ReplicatedActor2 = nullptr;
	};

	const int32 Actor1ExpectedReplicatedValue = 42;
	const int32 Actor2ExpectedReplicatedValue = 24;

	FPIENetworkComponent<DerivedState> Network{ TestRunner, TestCommandBuilder, bInitializing };

	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("0"));
	}

	BEFORE_EACH()
	{
		FNetworkComponentBuilder<DerivedState>()
			.WithClients(1)
			.Build(Network);
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	auto MakeSetInt(int32 Value) 
	{
		return [Value](ATestReplicatedActor& Actor) {
			Actor.ReplicatedInt = Value;
		};
	}

	TEST_METHOD(SpawnAndReplicateActor_WithMultipleActors_ReplicatesBoth)
	{
		Network.SpawnAndReplicate<ATestReplicatedActor, &DerivedState::ReplicatedActor1>(MakeSetInt(Actor1ExpectedReplicatedValue))
			.SpawnAndReplicate<ATestReplicatedActor, &DerivedState::ReplicatedActor2>(MakeSetInt(Actor2ExpectedReplicatedValue))
			.ThenServer([this](DerivedState& ServerState) {
				ASSERT_THAT(AreEqual(Actor1ExpectedReplicatedValue, ServerState.ReplicatedActor1->ReplicatedInt));
				ASSERT_THAT(AreEqual(Actor2ExpectedReplicatedValue, ServerState.ReplicatedActor2->ReplicatedInt));
			})
			.ThenClients([this](DerivedState& ClientState) {
				ASSERT_THAT(AreEqual(Actor1ExpectedReplicatedValue, ClientState.ReplicatedActor1->ReplicatedInt));
				ASSERT_THAT(AreEqual(Actor2ExpectedReplicatedValue, ClientState.ReplicatedActor2->ReplicatedInt));
			});
	}

	TEST_METHOD(SpawnAndReplicate_WithLateJoin_ReplicatesBoth)
	{
		Network.SpawnAndReplicate<ATestReplicatedActor, &DerivedState::ReplicatedActor1>(MakeSetInt(42))
			.SpawnAndReplicate<ATestReplicatedActor, &DerivedState::ReplicatedActor2>(MakeSetInt(24))
			.ThenServer([this](DerivedState& ServerState) {
				ASSERT_THAT(AreEqual(Actor1ExpectedReplicatedValue, ServerState.ReplicatedActor1->ReplicatedInt));
				ASSERT_THAT(AreEqual(Actor2ExpectedReplicatedValue, ServerState.ReplicatedActor2->ReplicatedInt));
			})
			.ThenClientJoins()
			.ThenClients([this](DerivedState& ClientState) {
				ASSERT_THAT(AreEqual(Actor1ExpectedReplicatedValue, ClientState.ReplicatedActor1->ReplicatedInt));
				ASSERT_THAT(AreEqual(Actor2ExpectedReplicatedValue, ClientState.ReplicatedActor2->ReplicatedInt));
			});
	}
};

NETWORK_TEST_CLASS(LateJoinTest, "TestFramework.CQTest.Network")
{
	struct DerivedState : public FBasePIENetworkComponentState
	{
		ATestReplicatedActor* ReplicatedActor = nullptr;
	};

	FPIENetworkComponent<DerivedState> Network{ TestRunner, TestCommandBuilder, bInitializing };

	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("0"));
	}

	BEFORE_EACH()
	{
		FNetworkComponentBuilder<DerivedState>()
			.WithGameInstanceClass(UGameInstance::StaticClass())
			.WithGameMode(AGameModeBase::StaticClass())
			.WithClients(1)
			.Build(Network);
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(ThenClientJoins_AfterStart_AddsClient) 
	{
		Network.ThenServer([this](DerivedState& ServerState) {
			ASSERT_THAT(AreEqual(ServerState.ClientCount, 1));
		})
		.ThenClientJoins()
		.ThenServer([this](DerivedState& ServerState) {
			ASSERT_THAT(AreEqual(ServerState.ClientCount, 2));
			ASSERT_THAT(AreEqual(ServerState.ClientConnections.Num(), 2));
			for (int32 ClientIndex = 0; ClientIndex < ServerState.ClientConnections.Num(); ++ClientIndex)
			{
				ASSERT_THAT(IsNotNull(ServerState.ClientConnections[ClientIndex]));
			}
		})
		.ThenClientJoins()
		.ThenServer([this](DerivedState& ServerState) {
			ASSERT_THAT(AreEqual(ServerState.ClientCount, 3));
			ASSERT_THAT(AreEqual(ServerState.ClientConnections.Num(), 3));
			for (int32 ClientIndex = 0; ClientIndex < ServerState.ClientConnections.Num(); ++ClientIndex)
			{
				ASSERT_THAT(IsNotNull(ServerState.ClientConnections[ClientIndex]));
			}
		});
	}

	TEST_METHOD(ThenClientJoins_AfterStart_ReplicatesState) 
	{
		Network.SpawnAndReplicate<ATestReplicatedActor, &DerivedState::ReplicatedActor>()
			.ThenServer([this](DerivedState& ServerState) {
				ASSERT_THAT(IsNotNull(ServerState.ReplicatedActor));
			})
			.ThenClient(0, [this](DerivedState& ClientState) {
				ASSERT_THAT(IsNotNull(ClientState.ReplicatedActor));
			})
			.ThenClientJoins()
			.ThenClient(1, [this](DerivedState& ClientState) {
				ASSERT_THAT(IsNotNull(ClientState.ReplicatedActor));
			});
	}
};

NETWORK_TEST_CLASS(GameInstanceTest, "TestFramework.CQTest.Network")
{
	const int32 ExpectedReplicatedValue = 42;

	FPIENetworkComponent<FBasePIENetworkComponentState> Network{ TestRunner, TestCommandBuilder, bInitializing };

	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("0"));
	}

	BEFORE_EACH()
	{
		FNetworkComponentBuilder()
			.WithGameInstanceClass(FSoftClassPath(UCQGameInstanceClass::StaticClass()))
			.WithGameMode(AGameModeBase::StaticClass())
			.Build(Network);
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(NetworkComponent_WithGameInstanceClass_BuildsNetworkWithProvidedGameInstance)
	{
		Network.ThenServer([this](FBasePIENetworkComponentState& ServerState) {
			UCQGameInstanceClass* GameInstance = Cast<UCQGameInstanceClass>(ServerState.World->GetGameInstance());
			ASSERT_THAT(IsNotNull(GameInstance));
			ASSERT_THAT(AreEqual(ExpectedReplicatedValue, GameInstance->TestValue));
		});
	}
};

NETWORK_TEST_CLASS(GameModeTest, "TestFramework.CQTest.Network")
{
	const int32 ExpectedReplicatedValue = 42;

	FPIENetworkComponent<> Network{ TestRunner, TestCommandBuilder, bInitializing };

	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("0"));
	}

	BEFORE_EACH()
	{
		FNetworkComponentBuilder()
			.WithGameInstanceClass(UGameInstance::StaticClass())
			.WithGameMode(ACQTestGameMode::StaticClass())
			.Build(Network);
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(NetworkComponent_WithGameMode_BuildsNetworkWithProvidedGameMode)
	{
		Network.ThenServer([this](FBasePIENetworkComponentState& ServerState) {
			ACQTestGameMode* GameMode = Cast<ACQTestGameMode>(ServerState.World->GetAuthGameMode());
			ASSERT_THAT(IsNotNull(GameMode));
			ASSERT_THAT(AreEqual(ExpectedReplicatedValue, GameMode->TestValue));
		});
	}
};

NETWORK_TEST_CLASS(SetupErrorTest, "TestFramework.CQTest.Network")
{
	FPIENetworkComponent<> Network{ TestRunner, TestCommandBuilder, bInitializing };

	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("0"));
	}

	AFTER_EACH() 
	{
		ClearExpectedError(*TestRunner, TEXT("Failed to initialize Network Component"));
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(NetworkComponent_WithoutUsingBuilder_AddsErrorAndDoesNotCrash)
	{
		Network.ThenServer([this](FBasePIENetworkComponentState&) { Assert.Fail("Unexpected Error"); });
	}
};

#endif // ENABLE_PIE_NETWORK_TEST