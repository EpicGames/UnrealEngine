// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "Components/ActorTestSpawner.h"

TEST_CLASS(SpawnHelperTests, "TestFramework.CQTest.Actor")
{
	FActorTestSpawner Spawner;

	TEST_METHOD(SpawnActor_WithNameParameter_ReturnsActorWithGivenName)
	{
		const FName ExpectedActorName = TEXT("AnyActorName");
		FActorSpawnParameters SpawnParameters;
		SpawnParameters.Name = ExpectedActorName;

		AActor& Actor = Spawner.SpawnActor<AActor>(SpawnParameters);

		ASSERT_THAT(AreEqual( Actor.GetFName(), ExpectedActorName ));
	}
	
	TEST_METHOD(AddComponentTo_SpawnedActor_SetsActorAsComponentOwner)
	{
		AActor& Owner = Spawner.SpawnActor<AActor>();
		auto* TestComponent = NewObject<USceneComponent>(&Owner);
		TestComponent->RegisterComponent();

		ASSERT_THAT(IsNotNull(TestComponent));
		ASSERT_THAT(AreEqual(TestComponent->GetOwner(), &Owner ));
	}

	TEST_METHOD(SpawnObject_WithAnObject_TearsDownCleanly)
	{
		auto& SomeComponent = Spawner.SpawnObject<USceneComponent>();
	}
};