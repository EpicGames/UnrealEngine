// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/TestReplicatedActor.h"

#include "Net/UnrealNetwork.h"

// -----------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestReplicatedActor)

ATestReplicatedActor::ATestReplicatedActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ReplicatedInt(0)
{
	bReplicates = true;
	bAlwaysRelevant = true;
}

// -----------------------------------------------------------------------------

void ATestReplicatedActor::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	DOREPLIFETIME(ATestReplicatedActor, ReplicatedInt);
}

// -----------------------------------------------------------------------------
// -----------------------------------------------------------------------------
// -----------------------------------------------------------------------------
