// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "TestReplicatedActor.generated.h"

// -----------------------------------------------------------------------------

UCLASS(NotBlueprintable)
class ATestReplicatedActor : public AActor
{
	GENERATED_BODY()

public:
	ATestReplicatedActor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;

	UPROPERTY(Replicated)
	int32 ReplicatedInt;
};

// -----------------------------------------------------------------------------
// -----------------------------------------------------------------------------
// -----------------------------------------------------------------------------
