// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "Assert/NoDiscardAsserter.h"

struct FCustomAsserter : public FNoDiscardAsserter
{
public:
	FCustomAsserter(FAutomationTestBase& testRunner)
		: FNoDiscardAsserter(testRunner)
	{
	}

	bool Custom(bool In) 
	{
		return In;
	}
};

#define CUSTOM_ASSERT_TEST_CLASS(_ClassName, _TestDir) TEST_CLASS_WITH_ASSERTS(_ClassName, _TestDir, FCustomAsserter)

CUSTOM_ASSERT_TEST_CLASS(CustomAsserts, "TestFramework.CQTest.Core")
{
	TEST_METHOD(CustomTestClass_WithCustomAsserter_HasInstanceOfCustomAsserter)
	{
		ASSERT_THAT(Custom(true));
		ASSERT_THAT(IsTrue(true));
	}
};

TEST_CLASS_WITH_ASSERTS_AND_TAGS(CustomAssertsTags, "TestFramework.CQTest.Core.Tags", FCustomAsserter, "[CQAssertTest][AssertExtraTag][CQTaggedTests]")
{
	TEST_METHOD(CustomTestClass_WithCustomAsserter_UsesCustomAsserter_HasExpectedTags)
	{
		ASSERT_THAT(Custom(true));
		
		const FString TestTags = TestRunner->GetTestTags();
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQAssertTest]"))));
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[AssertExtraTag]"))));
	}

	TEST_METHOD_WITH_TAGS(CustomTestClass_HasExpectedMethodTags, "[CQClassWithAssertsAndTagsMethod]")
	{
		const FString ThisMethodName = TEXT("CustomTestClass_HasExpectedMethodTags");
		const FString TestTags = TestRunner->GetTestMethodTags(ThisMethodName);
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQAssertTest]"))));
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[AssertExtraTag]"))));
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQClassWithAssertsAndTagsMethod]"))));
	}
};

template<typename Derived, typename AsserterType>
struct TCustomBaseClass : public TTest<Derived, AsserterType>
{
	inline static uint32 BaseValue = 0;
	uint32 SpecialValue = 42;

	BEFORE_ALL()
	{
		BaseValue = 42;
	}

	AFTER_ALL()
	{
		BaseValue = 0;
	}

	void AssertInBase()
	{
		ASSERT_THAT(AreEqual(42, SpecialValue));
	}
};

#define CUSTOM_BASE_TEST_CLASS(_ClassName, _TestDir) TEST_CLASS_WITH_BASE(_ClassName, _TestDir, TCustomBaseClass)

CUSTOM_BASE_TEST_CLASS(DerivedTest, "TestFramework.CQTest.Core")
{
	inline static uint32 DerivedValue = 0;

	BEFORE_ALL()
	{
		TCustomBaseClass::BeforeAll(FString());
		DerivedValue = BaseValue;
	}

	AFTER_ALL()
	{
		DerivedValue = 0;
		TCustomBaseClass::AfterAll(FString());
	}

	TEST_METHOD(DerivedTestClass_WithCustomBase_InheritsFromBaseClass)
	{
		ASSERT_THAT(AreEqual(42, SpecialValue));
	}

	TEST_METHOD(DerivedTestClass_WithBeforeAll_CanUseBaseBeforeAll)
	{
		ASSERT_THAT(AreEqual(42, BaseValue));
		ASSERT_THAT(AreEqual(BaseValue, DerivedValue));
	}

	TEST_METHOD(DerivedTestClass_WithFunctionThatAssertsInBase_CanAssertInBase)
	{
		AssertInBase();
	}
};

TEST_CLASS_WITH_BASE_AND_FLAGS(DerivedWithFlagsTest, "TestFramework.CQTest.Core", TCustomBaseClass, EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
{
	TEST_METHOD(DerivedTestClass_WithCustomBase_InheritsFromBaseClass_HasExpectedFlags)
	{
		ASSERT_THAT(AreEqual(42, SpecialValue));

		ASSERT_THAT(AreEqual(EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter, TestRunner->GetTestFlags()));
	}
};

TEST_CLASS_WITH_BASE_AND_TAGS(DerivedWithTagsTest, "TestFramework.CQTest.Core.Tags", TCustomBaseClass, "[CQBaseTest][BaseExtraTag][CQTaggedTests]")
{
	TEST_METHOD(DerivedTestClass_WithCustomBase_InheritsFromBaseClass_HasExpectedTags)
	{
		ASSERT_THAT(AreEqual(42, SpecialValue));

		const FString TestTags = TestRunner->GetTestTags();
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQBaseTest]"))));
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[BaseExtraTag]"))));
	}

	TEST_METHOD_WITH_TAGS(DerivedTestClass_HasExpectedMethodTags, "[CQTestClassWithBaseAndTagsMethod]")
	{
		const FString ThisMethodName = TEXT("DerivedTestClass_HasExpectedMethodTags");
		const FString TestTags = TestRunner->GetTestMethodTags(ThisMethodName);
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQBaseTest]"))));
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[BaseExtraTag]"))));
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQTestClassWithBaseAndTagsMethod]"))));
	}
};

TEST_CLASS_WITH_BASE_AND_FLAGS_AND_TAGS(DerivedWithFlagsAndTagsTest, "TestFramework.CQTest.Core.Tags", TCustomBaseClass, EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter, "[CQBaseFlagsTest][BaseFlagsExtraTag][CQTaggedTests]")
{
	TEST_METHOD(DerivedTestClass_WithCustomBase_InheritsFromBaseClass_HasExpectedFlags_HasExpectedTags)
	{
		ASSERT_THAT(AreEqual(42, SpecialValue));

		ASSERT_THAT(AreEqual(EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter, TestRunner->GetTestFlags()));

		const FString TestTags = TestRunner->GetTestTags();
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQBaseFlagsTest]"))));
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[BaseFlagsExtraTag]"))));
	}

	TEST_METHOD_WITH_TAGS(DerivedTestClass_HasExpectedMethodTags, "[CQTestClassWithBaseAndFlagsAndTagsMethod]")
	{
		const FString ThisMethodName = TEXT("DerivedTestClass_HasExpectedMethodTags");
		const FString TestTags = TestRunner->GetTestMethodTags(ThisMethodName);
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQBaseFlagsTest]"))));
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[BaseFlagsExtraTag]"))));
		ASSERT_THAT(IsTrue(TestTags.Contains(TEXT("[CQTestClassWithBaseAndFlagsAndTagsMethod]"))));
	}
};

template <typename Derived, typename AsserterType>
struct TBaseWithConstructor : public TTest<Derived, AsserterType>
{
	static inline bool bInitializedCall{ false };

	static inline bool bNonInitializedCall{ false };

	TBaseWithConstructor()
	{
		if (this->bInitializing)
		{
			bInitializedCall = true;
		}
		else
		{
			bNonInitializedCall = true;
		}
	}
};

#define CUSTOM_WITH_CTOR_CLASS(_ClassName, _TestDir) TEST_CLASS_WITH_BASE(_ClassName, _TestDir, TBaseWithConstructor)

CUSTOM_WITH_CTOR_CLASS(CustomCtor, "TestFramework.CQTest.Core")
{
	TEST_METHOD(CustomClassWithCtor_CallsCtorWhenInitializing_AndBeforeEachTest)
	{
		ASSERT_THAT(IsTrue(bInitializedCall));	 // called when populating test names
		ASSERT_THAT(IsTrue(bNonInitializedCall)); // called when creating this test
	}
};