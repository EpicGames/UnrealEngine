// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"

#if WITH_EDITOR && WITH_AUTOMATION_TESTS

#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "Helpers/CQTestAssetFilterBuilder.h"
#include "Helpers/CQTestAssetHelper.h"
#include "CQTestUnitTestHelper.h"
#include "ObjectTools.h"
#include "Tests/AutomationEditorCommon.h"

namespace {

constexpr auto AssetNameStr = TEXT("MyBlueprint");
constexpr auto ClassPathStr = TEXT("MyBlueprint");
constexpr auto PackageNameStr = TEXT("/Engine/CQTestAssetHelperTest/MyBlueprint");
constexpr auto PackagePathStr = TEXT("/Engine/CQTestAssetHelperTest");

const FTopLevelAssetPath AssetClassPath(TEXT("/Script/Engine"), TEXT("Blueprint"));

} //anonymous

TEST_CLASS_WITH_FLAGS(CQTestAssetHelperTests, "TestFramework.CQTest.Helpers", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	inline static FAssetData TestAssetData;

	BEFORE_ALL()
	{
		// Create the asset using AssetTools with the provided parameters
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
		UObject* TestObject = AssetTools.CreateAsset(AssetNameStr, PackagePathStr, UBlueprint::StaticClass(), nullptr);

		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		AssetRegistry.AssetCreated(TestObject);

		TestAssetData = FAssetData(TestObject);
	}

	AFTER_ALL()
	{
		UObject* TestObject = TestAssetData.GetAsset();
		FAutomationEditorCommonUtils::NullReferencesToObject(TestObject);
		bool bDeleted = ObjectTools::DeleteSingleObject(TestObject, false);
		checkf(bDeleted, TEXT("Could not delete test asset"));
		
		// Broadcast the class of the successfully deleted object
		// Reason: Ensures listeners are notified of the deletion event
		FEditorDelegates::OnAssetsDeleted.Broadcast({ TestObject->GetClass() });
	}

	TEST_METHOD(FindAssetPackageByName_WithPackageName_ReturnsPackagePath)
	{
		FARFilter Filter = CQTestAssetHelper::FAssetFilterBuilder()
			.WithPackageName(PackageNameStr)
			.Build();

		const TOptional<FString> PackagePath = CQTestAssetHelper::FindAssetPackagePathByName(Filter, AssetNameStr);
		ASSERT_THAT(IsTrue(PackagePath.IsSet()));
		ASSERT_THAT(AreEqual(PackagePath.GetValue(), TestAssetData.PackagePath));
	}

	TEST_METHOD(FindAssetPackageByName_WithPackagePath_ReturnsPackagePath)
	{
		FARFilter Filter = CQTestAssetHelper::FAssetFilterBuilder()
			.WithPackagePath(PackagePathStr)
			.Build();

		const TOptional<FString> PackagePath = CQTestAssetHelper::FindAssetPackagePathByName(Filter, AssetNameStr);
		ASSERT_THAT(IsTrue(PackagePath.IsSet()));
		ASSERT_THAT(AreEqual(PackagePath.GetValue(), PackagePathStr));
	}

	TEST_METHOD(FindAssetPackageByName_WithClassPath_ReturnsPackagePath)
	{
		FARFilter Filter =  CQTestAssetHelper::FAssetFilterBuilder()
			.WithClassPath(AssetClassPath)
			.Build();

		const TOptional<FString> PackagePath = CQTestAssetHelper::FindAssetPackagePathByName(Filter, AssetNameStr);
		ASSERT_THAT(IsTrue(PackagePath.IsSet()));
		ASSERT_THAT(AreEqual(PackagePath.GetValue(), PackagePathStr));
	}
	
	TEST_METHOD(FindAssetPackageByName_WithPartialPath_ReturnsPackagePath)
	{
		FARFilter Filter = CQTestAssetHelper::FAssetFilterBuilder()
			.WithPackagePath(TEXT("/Engine"))
			.IncludeRecursivePaths()
			.Build();

		const TOptional<FString> PackagePath = CQTestAssetHelper::FindAssetPackagePathByName(Filter, AssetNameStr);
		ASSERT_THAT(IsTrue(PackagePath.IsSet()));
		ASSERT_THAT(AreEqual(PackagePath.GetValue(), PackagePathStr));
	}
	
	TEST_METHOD(FindAssetPackageByName_WithMissingSlash_ReturnsPackagePath)
	{
		FARFilter Filter = CQTestAssetHelper::FAssetFilterBuilder()
			.WithPackagePath(TEXT("Engine"))
			.IncludeRecursivePaths()
			.Build();

		const TOptional<FString> PackagePath = CQTestAssetHelper::FindAssetPackagePathByName(Filter, AssetNameStr);
		ASSERT_THAT(IsTrue(PackagePath.IsSet()));
		ASSERT_THAT(AreEqual(PackagePath.GetValue(), PackagePathStr));
	}
	
	TEST_METHOD(FindAssetPackageByName_WithoutFilter_ReturnsPackagePath)
	{
		const TOptional<FString> PackagePath = CQTestAssetHelper::FindAssetPackagePathByName(AssetNameStr);
		ASSERT_THAT(IsTrue(PackagePath.IsSet()));
		ASSERT_THAT(AreEqual(PackagePath.GetValue(), PackagePathStr));
	}
	
	TEST_METHOD(FindAssetPackageByName_WrongName_ReturnsNullOpt) 
	{
		FARFilter Filter = CQTestAssetHelper::FAssetFilterBuilder()
			.WithPackageName(PackageNameStr)
			.Build();

		const FString& BadAssetName = TEXT("RandomAssetName");
		const TOptional<FString> PackagePath = CQTestAssetHelper::FindAssetPackagePathByName(Filter, BadAssetName);
		ClearExpectedWarning(*TestRunner, FString::Printf(TEXT("Asset name %s not found."), *BadAssetName));
		ASSERT_THAT(IsFalse(PackagePath.IsSet()));
	}
	
	TEST_METHOD(FindAssetPackageByName_WrongPackagePath_ReturnsNullOpt)
	{
		const FName IncorrectPackagePath = TEXT("/Wrong/Path");

		FARFilter Filter = CQTestAssetHelper::FAssetFilterBuilder()
			.WithPackagePath(IncorrectPackagePath)
			.Build();

		const TOptional<FString> PackagePath = CQTestAssetHelper::FindAssetPackagePathByName(Filter, AssetNameStr);
		ClearExpectedWarning(*TestRunner, FString::Printf(TEXT("Asset name %s not found."), AssetNameStr));
		ASSERT_THAT(IsFalse(PackagePath.IsSet()));
	}

	TEST_METHOD(GetBlueprintClass_WithFilter_ReturnsBlueprintClass)
	{
		FARFilter Filter = CQTestAssetHelper::FAssetFilterBuilder()
			.WithClassPath(AssetClassPath)
			.Build();

		UClass* Class = CQTestAssetHelper::GetBlueprintClass(Filter, AssetNameStr);
		ASSERT_THAT(IsNotNull(Class));
		ASSERT_THAT(AreEqual(Class->GetClassPathName(), AssetClassPath));
	}

	TEST_METHOD(FindDataBlueprint_WithFilter_ReturnsBlueprintObject)
	{
		FARFilter Filter = CQTestAssetHelper::FAssetFilterBuilder()
			.WithClassPath(AssetClassPath)
			.Build();

		UObject* TestObject = CQTestAssetHelper::FindDataBlueprint(Filter, AssetNameStr);
		ASSERT_THAT(IsTrue(IsValid(TestObject)));
		UBlueprint* TestBlueprint = Cast<UBlueprint>(TestObject);
		ASSERT_THAT(IsNotNull(TestObject));
	}

	TEST_METHOD(FindAssets_WithFilter_ReturnsAssetArray)
	{
		FARFilter Filter = CQTestAssetHelper::FAssetFilterBuilder()
			.WithPackageName(PackageNameStr)
			.Build();

		TArray<FAssetData> Assets = CQTestAssetHelper::FindAssetsByFilter(Filter);
		ASSERT_THAT(IsTrue(Assets.Num() == 1));

		UObject* TestObject = Assets[0].GetAsset();
		ASSERT_THAT(IsTrue(IsValid(TestObject)));
	}
};

#endif // WITH_EDITOR && WITH_AUTOMATION_TESTS