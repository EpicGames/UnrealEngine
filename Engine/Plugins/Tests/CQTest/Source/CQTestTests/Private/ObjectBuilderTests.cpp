// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestActorWithProperties.h"

#include "Animation/AnimSequence.h"
#include "Components/SceneComponent.h"

#include "ObjectBuilder.h"
#include "Components/ActorTestSpawner.h"
#include "CQTest.h"

#if WITH_AUTOMATION_WORKER

TEST_CLASS(ObjectBuilder_Success, "TestFramework.CQTest.Actor")
{
	FActorTestSpawner Spawner;

	TEST_METHOD(Spawn_NonActorWithProperties_SetProperties)
	{
		auto& Result = TObjectBuilder<UAnimSequence>()
						   .SetParam<bool>("bEnableRootMotion", true)
						   .SetParam("RefFrameIndex", 321)
						   .Spawn();

		ASSERT_THAT(IsTrue(Result.bEnableRootMotion));
		ASSERT_THAT(AreEqual(Result.RefFrameIndex, 321));
	}
	TEST_METHOD(Spawn_ActorWithBool_SetsBooleanValue)
	{
		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner).SetParam("BoolProperty", true).Spawn();
		ASSERT_THAT(IsTrue(Result.BoolProperty));
		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner).SetParam<const bool>("BoolProperty", true).Spawn();
		ASSERT_THAT(IsTrue(ConstResult.BoolProperty));
	}
	TEST_METHOD(Spawn_ActorWithByte_SetsByteValue)
	{
		uint8 Byte = 250;
		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner).SetParam("ByteProperty", Byte).Spawn();
		ASSERT_THAT(AreEqual(Result.ByteProperty, Byte));
		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner).SetParam<const uint8>("ByteProperty", Byte).Spawn();
		ASSERT_THAT(AreEqual(ConstResult.ByteProperty, Byte));
	}

	TEST_METHOD(Spawn_ActorWithIntegers_SetsIntegers)
	{
		int8 int8Value = -112;
		int16 int16Value = 10000;
		int32 int32Value = -1000000;
		int64 int64Value = -10000000;
		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("Int8Property", int8Value)
						   .SetParam("Int16Property", int16Value)
						   .SetParam("Int32Property", int32Value)
						   .SetParam("Int64Property", int64Value)
						   .Spawn();

		ASSERT_THAT(AreEqual(Result.Int8Property, int8Value));
		ASSERT_THAT(AreEqual(Result.Int16Property, int16Value));
		ASSERT_THAT(AreEqual(Result.Int32Property, int32Value));
		ASSERT_THAT(AreEqual(Result.Int64Property, int64Value));

		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
								.SetParam<const int8>("Int8Property", int8Value)
								.SetParam<const int16>("Int16Property", int16Value)
								.SetParam<const int32>("Int32Property", int32Value)
								.SetParam<const int64>("Int64Property", int64Value)
								.Spawn();

		ASSERT_THAT(AreEqual(ConstResult.Int8Property, int8Value));
		ASSERT_THAT(AreEqual(ConstResult.Int16Property, int16Value));
		ASSERT_THAT(AreEqual(ConstResult.Int32Property, int32Value));
		ASSERT_THAT(AreEqual(ConstResult.Int64Property, int64Value));
	}

	TEST_METHOD(Spawn_ActorWithUnsignedIntegers_SetsUnsignedIntegers)
	{
		uint16 uint16Value = 10000;
		uint32 uint32Value = 5000000;
		uint64 uint64Value = 400000000;

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("UInt16Property", uint16Value)
						   .SetParam("UInt32Property", uint32Value)
						   .SetParam("UInt64Property", uint64Value)
						   .Spawn();

		ASSERT_THAT(AreEqual(Result.UInt16Property, uint16Value));
		ASSERT_THAT(AreEqual(Result.UInt32Property, uint32Value));
		ASSERT_THAT(AreEqual(Result.UInt64Property, uint64Value));

		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const uint16>("UInt16Property", uint16Value)
						   .SetParam<const uint32>("UInt32Property", uint32Value)
						   .SetParam<const uint64>("UInt64Property", uint64Value)
						   .Spawn();

		ASSERT_THAT(AreEqual(ConstResult.UInt16Property, uint16Value));
		ASSERT_THAT(AreEqual(ConstResult.UInt32Property, uint32Value));
		ASSERT_THAT(AreEqual(ConstResult.UInt64Property, uint64Value));
	}

	TEST_METHOD(Spawn_ActorWithFloats_SetsFloats)
	{
		float floatValue = 1.23f;
		double doubleValue = 1.23456789;

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("FloatProperty", floatValue)
						   .SetParam("DoubleProperty", doubleValue)
						   .Spawn();

		ASSERT_THAT(IsTrue(Result.FloatProperty == floatValue));
		ASSERT_THAT(IsTrue(Result.DoubleProperty == doubleValue));

		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const float>("FloatProperty", floatValue)
						   .SetParam<const double>("DoubleProperty", doubleValue)
						   .Spawn();

		ASSERT_THAT(IsTrue(Result.FloatProperty == floatValue));
		ASSERT_THAT(IsTrue(Result.DoubleProperty == doubleValue));
	}

	TEST_METHOD(Spawn_ActorWithEnums_SetsEnums)
	{
		ETestInt8 s8Enum = ETestInt8::enumone;
		ETestInt16 s16Enum = ETestInt16::enumtwo;
		ETestInt32 s32Enum = ETestInt32::enumthree;
		ETestInt64 s64Enum = ETestInt64::enumtwo;

		ETestUint8 u8Enum = ETestUint8::enumthree;
		ETestUint16 u16Enum = ETestUint16::enumone;
		ETestUint32 u32Enum = ETestUint32::enumtwo;
		ETestUint64 u64Enum = ETestUint64::enumone;

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("Int8EnumProperty", s8Enum)
						   .SetParam("Int16EnumProperty", s16Enum)
						   .SetParam("Int32EnumProperty", s32Enum)
						   .SetParam("Int64EnumProperty", s64Enum)
						   .SetParam("Uint8EnumProperty", u8Enum)
						   .SetParam("Uint16EnumProperty", u16Enum)
						   .SetParam("Uint32EnumProperty", u32Enum)
						   .SetParam("Uint64EnumProperty", u64Enum)
						   .Spawn();

		ASSERT_THAT(AreEqual(Result.Int8EnumProperty, s8Enum));
		ASSERT_THAT(AreEqual(Result.Int16EnumProperty, s16Enum));
		ASSERT_THAT(AreEqual(Result.Int32EnumProperty, s32Enum));
		ASSERT_THAT(AreEqual(Result.Int64EnumProperty, s64Enum));
		ASSERT_THAT(AreEqual(Result.Uint8EnumProperty, u8Enum));
		ASSERT_THAT(AreEqual(Result.Uint16EnumProperty, u16Enum));
		ASSERT_THAT(AreEqual(Result.Uint32EnumProperty, u32Enum));
		ASSERT_THAT(AreEqual(Result.Uint64EnumProperty, u64Enum));

		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const ETestInt8>("Int8EnumProperty", s8Enum)
						   .SetParam<const ETestInt16>("Int16EnumProperty", s16Enum)
						   .SetParam<const ETestInt32>("Int32EnumProperty", s32Enum)
						   .SetParam<const ETestInt64>("Int64EnumProperty", s64Enum)
						   .SetParam<const ETestUint8>("Uint8EnumProperty", u8Enum)
						   .SetParam<const ETestUint16>("Uint16EnumProperty", u16Enum)
						   .SetParam<const ETestUint32>("Uint32EnumProperty", u32Enum)
						   .SetParam<const ETestUint64>("Uint64EnumProperty", u64Enum)
						   .Spawn();

		ASSERT_THAT(AreEqual(ConstResult.Int8EnumProperty, s8Enum));
		ASSERT_THAT(AreEqual(ConstResult.Int16EnumProperty, s16Enum));
		ASSERT_THAT(AreEqual(ConstResult.Int32EnumProperty, s32Enum));
		ASSERT_THAT(AreEqual(ConstResult.Int64EnumProperty, s64Enum));
		ASSERT_THAT(AreEqual(ConstResult.Uint8EnumProperty, u8Enum));
		ASSERT_THAT(AreEqual(ConstResult.Uint16EnumProperty, u16Enum));
		ASSERT_THAT(AreEqual(ConstResult.Uint32EnumProperty, u32Enum));
		ASSERT_THAT(AreEqual(ConstResult.Uint64EnumProperty, u64Enum));
	}

	TEST_METHOD(Spawn_WithFName_SetsFName)
	{
		FName Name = FName(TEXT("TestName"));

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("NameProperty", Name)
						   .Spawn();

		ASSERT_THAT(AreEqual(Result.NameProperty, Name));

		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const FName>("NameProperty", Name)
						   .Spawn();

		ASSERT_THAT(AreEqual(ConstResult.NameProperty, Name));

	}

	TEST_METHOD(Spawn_WithTObjectPtrProperty_SetsProperty)
	{
		auto& BuiltObject = TObjectBuilder<UAnimSequence>().Spawn();
		auto ptr = TObjectPtr<UAnimSequence>(&BuiltObject);

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("TestTObjectPtrProperty", ptr)
						   .Spawn();

		ASSERT_THAT(IsTrue(Result.TestTObjectPtrProperty == ptr));

		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const TObjectPtr<UAnimSequence>>("TestTObjectPtrProperty", ptr)
						   .Spawn();

		ASSERT_THAT(IsTrue(ConstResult.TestTObjectPtrProperty == ptr));
	}

	TEST_METHOD(Spawn_WithVectorProperty_SetsProperty)
	{
		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("VectorProperty", FVector::UpVector)
						   .Spawn();

		ASSERT_THAT(AreEqual(Result.VectorProperty, FVector::UpVector));


		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const FVector>("VectorProperty", FVector::UpVector)
						   .Spawn();

		ASSERT_THAT(AreEqual(ConstResult.VectorProperty, FVector::UpVector));
	}

	TEST_METHOD(Spawn_WithStructProperty_SetsProperty)
	{
		TArray<int32> ArrayInStruct;
		ArrayInStruct.Add(123);
		ArrayInStruct.Add(456);
		ArrayInStruct.Add(789);

		FTestStructWithProperties UnrealStruct;
		UnrealStruct.StructInt32Property = 321;
		UnrealStruct.StructArrayProperty = ArrayInStruct;

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("StructProperty", UnrealStruct)
						   .Spawn();

		ASSERT_THAT(AreEqual(Result.StructProperty.StructInt32Property, 321));
		ASSERT_THAT(AreEqual(Result.StructProperty.StructArrayProperty, ArrayInStruct));


		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const FTestStructWithProperties>("StructProperty", UnrealStruct)
						   .Spawn();

		ASSERT_THAT(AreEqual(ConstResult.StructProperty.StructInt32Property, 321));
		ASSERT_THAT(AreEqual(ConstResult.StructProperty.StructArrayProperty, ArrayInStruct));
	}

	TEST_METHOD(Spawn_WithPrimitiveTArray_SetsArray)
	{
		TArray<int32> Array;
		Array.Add(1);
		Array.Add(2);
		Array.Add(3);
		Array.Add(4);

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("ArrayProperty", Array)
						   .Spawn();

		ASSERT_THAT(IsTrue(Result.ArrayProperty == Array));

		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const TArray<int32>>("ArrayProperty", Array)
						   .Spawn();

		ASSERT_THAT(IsTrue(ConstResult.ArrayProperty == Array));
	}

	TEST_METHOD(Spawn_WithObjectArray_SetsArray)
	{
		TArray<TObjectPtr<ATestActorWithProperties>> ObjectArray;
		ObjectArray.Add(&TObjectBuilder<ATestActorWithProperties>(Spawner).Spawn());

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("ArrayOfObjectsProperty", ObjectArray)
						   .Spawn();

		ASSERT_THAT(IsTrue(Result.ArrayOfObjectsProperty == ObjectArray));

		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const TArray<ATestActorWithProperties*>>("ArrayOfObjectsProperty", ObjectArray)
						   .Spawn();

		ASSERT_THAT(IsTrue(ConstResult.ArrayOfObjectsProperty == ObjectArray));
	}

	TEST_METHOD(Spawn_WithStructArray_SetsProperty)
	{
		TArray<int32> ArrayInStruct;
		ArrayInStruct.Add(123);
		ArrayInStruct.Add(456);
		ArrayInStruct.Add(789);

		FTestStructWithProperties UnrealStruct;
		UnrealStruct.StructInt32Property = 321;
		UnrealStruct.StructArrayProperty = ArrayInStruct;

		TArray<int32> Array;
		Array.Add(1);
		Array.Add(2);
		Array.Add(3);
		Array.Add(4);

		FTestStructWithProperties OtherUnrealStruct;
		OtherUnrealStruct.StructInt32Property = 426;
		OtherUnrealStruct.StructArrayProperty = Array;

		TArray<FTestStructWithProperties> StructArray;
		StructArray.Add(UnrealStruct);
		StructArray.Add(OtherUnrealStruct);

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("ArrayOfStructsProperty", StructArray)
						   .Spawn();

		ASSERT_THAT(AreEqual(Result.ArrayOfStructsProperty.Num(), 2));
		ASSERT_THAT(IsTrue(Result.ArrayOfStructsProperty[0].StructInt32Property == 426 || Result.ArrayOfStructsProperty[1].StructInt32Property == 426));
		ASSERT_THAT(IsTrue(Result.ArrayOfStructsProperty[0].StructInt32Property == 321 || Result.ArrayOfStructsProperty[1].StructInt32Property == 321));
		ASSERT_THAT(AreNotEqual(Result.ArrayOfStructsProperty[0].StructInt32Property, Result.ArrayOfStructsProperty[1].StructInt32Property));


		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const TArray<FTestStructWithProperties>>("ArrayOfStructsProperty", StructArray)
						   .Spawn();

		ASSERT_THAT(AreEqual(ConstResult.ArrayOfStructsProperty.Num(), 2));
		ASSERT_THAT(IsTrue(ConstResult.ArrayOfStructsProperty[0].StructInt32Property == 426 || Result.ArrayOfStructsProperty[1].StructInt32Property == 426));
		ASSERT_THAT(IsTrue(ConstResult.ArrayOfStructsProperty[0].StructInt32Property == 321 || Result.ArrayOfStructsProperty[1].StructInt32Property == 321));
		ASSERT_THAT(AreNotEqual(ConstResult.ArrayOfStructsProperty[0].StructInt32Property, Result.ArrayOfStructsProperty[1].StructInt32Property));
	}

	TEST_METHOD(Spawn_WithVectorArray_SetsArray)
	{
		TArray<FVector> VectorArray;
		VectorArray.Add(FVector::UpVector);
		VectorArray.Add(FVector::LeftVector);

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("ArrayOfVectorsProperty", VectorArray)
						   .Spawn();

		ASSERT_THAT(IsTrue(Result.ArrayOfVectorsProperty == VectorArray));


		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const TArray<FVector>>("ArrayOfVectorsProperty", VectorArray)
						   .Spawn();

		ASSERT_THAT(IsTrue(ConstResult.ArrayOfVectorsProperty == VectorArray));
	}

	TEST_METHOD(Spawn_WithSet_SetsSet)
	{
		TSet<int32> Set;
		Set.Add(1);

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("SetProperty", Set)
						   .Spawn();

		ASSERT_THAT(IsNotNull(Result.SetProperty.Find(1)));

		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const TSet<int32>>("SetProperty", Set)
						   .Spawn();

		ASSERT_THAT(IsNotNull(ConstResult.SetProperty.Find(1)));
	}

	TEST_METHOD(Spawn_WithMapProperty_SetsMap)
	{
		TMap<int32, int32> Map;
		Map.Add(1, 2);
		Map.Add(3, 4);
		Map.Add(5, 6);

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam("MapProperty", Map)
						   .Spawn();

		ASSERT_THAT(IsTrue(Result.MapProperty.OrderIndependentCompareEqual(Map)));

		auto& ConstResult = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .SetParam<const TMap<int32, int32>>("MapProperty", Map)
						   .Spawn();

		ASSERT_THAT(IsTrue(ConstResult.MapProperty.OrderIndependentCompareEqual(Map)));
	}

	TEST_METHOD(Spawn_WithComponentType_AddsComponent)
	{
		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .AddComponentTo<USceneComponent>()
						   .Spawn();

		ASSERT_THAT(AreEqual(Result.GetComponents().Num(), 1));
	}

	TEST_METHOD(Spawn_WithBuiltComponent_AddsComponentInPreBuiltState)
	{
		const FVector Location(1.0f, 2.0f, 3.0f);
		auto& BuiltComponent = TObjectBuilder<USceneComponent>()
								   .SetParam("RelativeLocation", Location)
								   .Spawn();

		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .AddComponentTo(&BuiltComponent)
						   .Spawn();

		auto Index = Result.GetComponents().FindId(&BuiltComponent);
		ASSERT_THAT(IsTrue(Index.IsValidId()));

		auto* Component = Cast<USceneComponent>(Result.GetComponents()[Index]);
		ASSERT_THAT(IsNotNull(Component));
		ASSERT_THAT(AreEqual(Component->GetRelativeLocation(), Location));
	}

	TEST_METHOD(Spawn_WithChildActor_AddsChildActor)
	{
		auto& Result = TObjectBuilder<ATestActorWithProperties>(Spawner)
						   .AddChildActorComponentTo<AActor>()
						   .Spawn();

		ASSERT_THAT(AreEqual(Result.GetComponents().Num(), 1));
	}

	TEST_METHOD(SpawnObject_WithNoOwner_MatchesDefaultOwner)
	{
		auto* NormalObject = NewObject<USceneComponent>();
		auto& BuiltObject = TObjectBuilder<USceneComponent>().Spawn();

		ASSERT_THAT(AreEqual(NormalObject->GetOwner(), BuiltObject.GetOwner()));
	}

	TEST_METHOD(SpawnObject_WithOwner_SetsOwner)
	{
		auto* Actor = &Spawner.SpawnActor<AActor>();
		auto& Component = TObjectBuilder<USceneComponent>(Actor).Spawn();

		ASSERT_THAT(AreEqual(Component.GetOwner(), Actor));
	}
};

TEST_CLASS(ObjectBuilder_ApiError, "TestFramework.CQTest.Actor")
{
	FActorTestSpawner Spawner;

	TEST_METHOD(SpawnCalled_AfterSpawn_Actor_Errors)
	{
		auto Builder = TObjectBuilder<ATestActorWithProperties>(Spawner);
		auto& first = Builder.Spawn();

		Assert.ExpectError("");
		Builder.Spawn();
	}

	TEST_METHOD(SpawnCalled_AfterSpawn_Object_Errors)
	{
		auto Builder = TObjectBuilder<UAnimSequence>();
		auto& BuiltObject = Builder.Spawn();

		Assert.ExpectError("");
		Builder.Spawn();
	}

	TEST_METHOD(SetParam_MissingProperty_Errors)
	{
		Assert.ExpectError("Failed to find Object* property");
		UObject* Object = nullptr;

		TObjectBuilder<AActor>(Spawner)
			.SetParam("MissingParameter", Object)
			.Spawn();
	}
};

TEST_CLASS(ObjectBuilder_PrimitiveParameterErrors, "TestFramework.CQTest.Actor")
{
	FActorTestSpawner Spawner;

	TEST_METHOD(SetParam_TypeMismatch_Integers_Errors)
	{
		Assert.ExpectError("Type mismatch", 4);

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("Int8Property", 1.f)
			.SetParam("Int16Property", 2.f)
			.SetParam("Int32Property", 3.f)
			.SetParam("Int64Property", 4.f)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_Unsigned_Errors)
	{
		Assert.ExpectError("Type mismatch", 4);

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("UInt8Property", 1.f)
			.SetParam("UInt16Property", 2.f)
			.SetParam("UInt32Property", 3.f)
			.SetParam("UInt64Property", 4.f)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_Bool_Errors)
	{
		Assert.ExpectError("Type mismatch");

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("BoolProperty", 42)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_Byte_Errors)
	{
		Assert.ExpectError("Type mismatch");

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("ByteProperty", 42)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_FloatingPoint_Errors)
	{
		Assert.ExpectError("Type mismatch", 2);

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("FloatProperty", 42)
			.SetParam("DoubleProperty", 42)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_FName_Errors)
	{
		Assert.ExpectError("Type mismatch");

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("NameProperty", 42)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_Arrays_Errors)
	{
		Assert.ExpectError("Type mismatch", 2);

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("ArrayProperty", 42)
			.SetParam("ArrayOfObjectsProperty", 42)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_Map_Errors)
	{
		Assert.ExpectError("Type mismatch");

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("MapProperty", 42)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_Struct_Errors)
	{
		Assert.ExpectError("Type mismatch");

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("StructProperty", 42)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_Vector_Errors)
	{
		Assert.ExpectError("Type mismatch");

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("VectorProperty", 42)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_Enums_Errors)
	{
		Assert.ExpectError("Type mismatch", 3);

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("Uint8EnumProperty", 0.0)
			.SetParam("Int16EnumProperty", 1.f)
			.SetParam("Uint32EnumProperty", FName("Wrong"))
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_EnumSize_Errors)
	{
		Assert.ExpectError("Type mismatch", 8);

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("Uint8EnumProperty", ETestUint16::enumone)
			.SetParam("Uint16EnumProperty", ETestUint8::enumone)
			.SetParam("Uint32EnumProperty", ETestUint64::enumone)
			.SetParam("Uint64EnumProperty", ETestUint32::enumone)
			.SetParam("Int8EnumProperty", ETestInt16::enumone)
			.SetParam("Int16EnumProperty", ETestInt8::enumone)
			.SetParam("Int32EnumProperty", ETestInt64::enumone)
			.SetParam("Int64EnumProperty", ETestInt32::enumone)
			.Spawn();
	}

	TEST_METHOD(SetParam_TypeMismatch_TestTObjectPtr_Errors)
	{
		Assert.ExpectError("Type mismatch");

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("TestTObjectPtrProperty", 123)
			.Spawn();
	}
};

TEST_CLASS(ObjectBuilder_StructParameterErrors, "TestFramework.CQTest.Actor")
{
	FActorTestSpawner Spawner;

	TEST_METHOD(SetParam_ObjectWithNullPointer_Errors)
	{
		UObject* NullObject = nullptr;
		Assert.ExpectError("Type mismatch");

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("TestTObjectPtrProperty", NullObject)
			.Spawn();
	}

	TEST_METHOD(SetParam_UnrelatedObject_Errors)
	{
		UActorComponent* UnrelatedStruct = nullptr;
		Assert.ExpectError("Type mismatch");

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("TestTObjectPtrProperty", UnrelatedStruct)
			.Spawn();
	}

	TEST_METHOD(SetParam_UnrelatedStruct_Errors)
	{
		FOtherTestStruct UnrelatedStructType;
		Assert.ExpectError("Type mismatch");

		auto& BuiltActor = TObjectBuilder<ATestActorWithProperties>(Spawner)
							   .SetParam("StructProperty", UnrelatedStructType)
							   .Spawn();
	}

	TEST_METHOD(SetParam_DerivedStruct_Errors)
	{
		FDerivedTestStruct Derived;
		Assert.ExpectError("Type mismatch");

		auto& BuiltActor = TObjectBuilder<ATestActorWithProperties>(Spawner)
							   .SetParam("StructProperty", Derived)
							   .Spawn();
	}
};


TEST_CLASS(ObjectBuilder_ArrayPropertyErrors, "TestFramework.CQTest.Actor")
{
	FActorTestSpawner Spawner;

	template <typename T>
	void Try(T t, bool bIsMap = false)
	{
		Assert.ExpectError(bIsMap? "Failed to find" : "Type mismatch", 4);

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("ArrayProperty", t)
			.SetParam("ArrayOfObjectsProperty", t)
			.SetParam("ArrayOfVectorsProperty", t)
			.SetParam("ArrayOfStructsProperty", t)
			.Spawn();
	}

	TEST_METHOD(SetParam_WithPrimitive_Errors)
	{
		Try(123);
	}

	TEST_METHOD(SetParam_WithUnrelatedSimpleArray_Errors)
	{
		TArray<float> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithUObjectArray_Errors)
	{
		TArray<TObjectPtr<UActorComponent>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithNestedSimpleArray_Errors)
	{
		TArray<TArray<float>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithNestedUObjectArray_Errors)
	{
		TArray<TArray<TObjectPtr<UActorComponent>>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithArrayOfStructs_Errors)
	{
		TArray<FOtherTestStruct> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithArrayOfMaps_Errors)
	{
		TArray<TMap<uint32, uint32>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithArrayOfVectors_Errors)
	{
		TArray<TMap<uint32, uint32>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithMap_Errors)
	{
		TMap<uint32, uint32> Unrelated;
		Try(Unrelated, true);
	}

	TEST_METHOD(SetParam_WithObjectIntMap_Errors)
	{
		TMap<UObject*, uint32> Unrelated;
		Try(Unrelated, true);
	}

	TEST_METHOD(SetParam_WithIntObjectMap_Errors)
	{
		TMap<uint32, UObject*> Unrelated;
		Try(Unrelated, true);
	}

	TEST_METHOD(SetParam_WithObjectObjectMap_Errors)
	{
		TMap<UObject*, UObject*> Unrelated;
		Try(Unrelated, true);
	}

	TEST_METHOD(SetParam_WithIntArrayMap_Errors)
	{
		TMap<uint32, TArray<UObject*>> Unrelated;
		Try(Unrelated, true);
	}

	TEST_METHOD(SetParam_WithU8Enum_Errors)
	{
		ETestUint8 Unrelated = ETestUint8::enumtwo;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithU16Enum_Errors)
	{
		ETestUint16 Unrelated = ETestUint16::enumone;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithU32Enum_Errors)
	{
		ETestUint32 Unrelated = ETestUint32::enumtwo;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithU64Enum_Errors)
	{
		ETestUint64 Unrelated = ETestUint64::enumone;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS8Enum_Errors)
	{
		ETestInt8 Unrelated = ETestInt8::enumone;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS16Enum_Errors)
	{
		ETestInt16 Unrelated = ETestInt16::enumtwo;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS32Enum_Errors)
	{
		ETestInt32 Unrelated = ETestInt32::enumthree;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS64Enum_Errors)
	{
		ETestInt64 Unrelated = ETestInt64::enumtwo;
		Try(Unrelated);
	}
};

TEST_CLASS(ObjectBuilder_SetPropertyErrors, "TestFramework.CQTest.Actor")
{
	FActorTestSpawner Spawner;

	template <typename T>
	void Try(T t, bool bIsMap = false)
	{
		Assert.ExpectError(bIsMap? "Failed to find" : "Type mismatch");
		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("SetProperty", t)
			.Spawn();
	}

		TEST_METHOD(SetParam_WithPrimitive_Errors)
	{
		Try(123);
	}

	TEST_METHOD(SetParam_WithUnrelatedSimpleArray_Errors)
	{
		TArray<float> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithUObjectArray_Errors)
	{
		TArray<UActorComponent*> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithNestedSimpleArray_Errors)
	{
		TArray<TArray<float>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithNestedUObjectArray_Errors)
	{
		TArray<TArray<UActorComponent*>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithArrayOfStructs_Errors)
	{
		TArray<FTestStructWithProperties> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithArrayOfMaps_Errors)
	{
		TArray<TMap<uint32, uint32>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithArrayOfVectors_Errors)
	{
		TArray<TMap<uint32, uint32>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithMapWrongKey_Errors)
	{
		TMap<UObject*, int32> Unrelated;
		Try(Unrelated, true);
	}

	TEST_METHOD(SetParam_WithMapWrongValue_Errors)
	{
		TMap<int32, UObject*> Unrelated;
		Try(Unrelated, true);
	}

	TEST_METHOD(SetParam_WithMapWrongKeyAndValue_Errors)
	{
		TMap<UObject*, UObject*> Unrelated;
		Try(Unrelated, true);
	}

	TEST_METHOD(SetParam_WithIntArrayMap_Errors)
	{
		TMap<uint32, TArray<UObject*>> Unrelated;
		Try(Unrelated, true);
	}

	TEST_METHOD(SetParam_WithU8Enum_Errors)
	{
		ETestUint8 Unrelated = ETestUint8::enumtwo;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithU16Enum_Errors)
	{
		ETestUint16 Unrelated = ETestUint16::enumone;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithU32Enum_Errors)
	{
		ETestUint32 Unrelated = ETestUint32::enumtwo;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithU64Enum_Errors)
	{
		ETestUint64 Unrelated = ETestUint64::enumone;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS8Enum_Errors)
	{
		ETestInt8 Unrelated = ETestInt8::enumone;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS16Enum_Errors)
	{
		ETestInt16 Unrelated = ETestInt16::enumtwo;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS32Enum_Errors)
	{
		ETestInt32 Unrelated = ETestInt32::enumthree;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS64Enum_Errors)
	{
		ETestInt64 Unrelated = ETestInt64::enumtwo;
		Try(Unrelated);
	}
};

TEST_CLASS(ObjectBuilder_MapPropertyErrors, "TestFramework.CQTest.Actor")
{
	FActorTestSpawner Spawner;

	template <typename T>
	void Try(T t)
	{
		Assert.ExpectError("Type mismatch");

		TObjectBuilder<ATestActorWithProperties>(Spawner)
			.SetParam("MapProperty", t)
			.Spawn();
	}

	TEST_METHOD(SetParam_WithPrimitive_Errors)
	{
		Try(123);
	}

	TEST_METHOD(SetParam_WithUnrelatedSimpleArray_Errors)
	{
		TArray<float> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithUObjectArray_Errors)
	{
		TArray<UActorComponent*> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithNestedSimpleArray_Errors)
	{
		TArray<TArray<float>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithNestedUObjectArray_Errors)
	{
		TArray<TArray<UActorComponent*>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithArrayOfStructs_Errors)
	{
		TArray<FTestStructWithProperties> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithArrayOfMaps_Errors)
	{
		TArray<TMap<uint32, uint32>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithArrayOfVectors_Errors)
	{
		TArray<TMap<uint32, uint32>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithMapWrongKey_Errors)
	{
		TMap<UObject*, int32> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithMapWrongValue_Errors)
	{
		TMap<int32, UObject*> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithMapWrongKeyAndValue_Errors)
	{
		TMap<UObject*, UObject*> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithIntArrayMap_Errors)
	{
		TMap<uint32, TArray<UObject*>> Unrelated;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithU8Enum_Errors)
	{
		ETestUint8 Unrelated = ETestUint8::enumtwo;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithU16Enum_Errors)
	{
		ETestUint16 Unrelated = ETestUint16::enumone;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithU32Enum_Errors)
	{
		ETestUint32 Unrelated = ETestUint32::enumtwo;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithU64Enum_Errors)
	{
		ETestUint64 Unrelated = ETestUint64::enumone;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS8Enum_Errors)
	{
		ETestInt8 Unrelated = ETestInt8::enumone;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS16Enum_Errors)
	{
		ETestInt16 Unrelated = ETestInt16::enumtwo;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS32Enum_Errors)
	{
		ETestInt32 Unrelated = ETestInt32::enumthree;
		Try(Unrelated);
	}

	TEST_METHOD(SetParam_WithS64Enum_Errors)
	{
		ETestInt64 Unrelated = ETestInt64::enumtwo;
		Try(Unrelated);
	}
};

TEST_CLASS(ObjectBuilder_PostSpawnErrors, "TestFramework.CQTest.Actor")
{
	FActorTestSpawner Spawner;

	template <typename T>
	void Try(const FName& Name, T t)
	{
		TObjectBuilder<ATestActorWithProperties> Builder(Spawner);
		Builder.Spawn();

		Assert.ExpectError(TEXT("Tried to SetParam"));

		Builder.SetParam(Name, t);
	}

	TEST_METHOD(SetIntProperty_PostSpawn_Errors)
	{
		Try("IntProperty", 42);
	}

	TEST_METHOD(SetMapProperty_PostSpawn_Errors)
	{
		TMap<int32, int32> Map;
		Try("MapProperty", Map);
	}

	TEST_METHOD(SetArrayProperty_PostSpawn_Errors)
	{
		TArray<int32> Array;
		Try("ArrayProperty", Array);
	}

	TEST_METHOD(SetObjectProperty_PostSpawn_Errors)
	{
		UObject* Object = nullptr;
		Try("UObjectProperty", Object);
	}

	TEST_METHOD(SetStructProperty_PostSpawn_Errors)
	{
		FTestStructWithProperties Struct;
		Try("UStructProperty", Struct);
	}

	TEST_METHOD(SetEnumProperty_PostSpawn_Errors)
	{
		TObjectBuilder<ATestActorWithProperties> Builder(Spawner);
		Builder.Spawn();

		Assert.ExpectError(TEXT("Tried to SetParam"), 8);

		Builder.SetParam("Uint8EnumProperty", ETestUint8::enumone);
		Builder.SetParam("Uint16EnumProperty", ETestUint16::enumone);
		Builder.SetParam("Uint32EnumProperty", ETestUint32::enumone);
		Builder.SetParam("Uint64EnumProperty", ETestUint64::enumone);

		Builder.SetParam("Int8EnumProperty", ETestInt8::enumone);
		Builder.SetParam("Int16EnumProperty", ETestInt16::enumone);
		Builder.SetParam("Int32EnumProperty", ETestInt32::enumone);
		Builder.SetParam("Int64EnumProperty", ETestInt64::enumone);
	}

	TEST_METHOD(AddComponentTo_PostSpawn_Errors)
	{
		TObjectBuilder<ATestActorWithProperties> Builder(Spawner);
		Builder.Spawn();

		Assert.ExpectError(TEXT("Tried to AddComponentTo"));
		Builder.AddComponentTo<UActorComponent>();
	}

	TEST_METHOD(AddChildActorComponentTo_PostSpawn_Errors)
	{
		TObjectBuilder<ATestActorWithProperties> Builder(Spawner);
		Builder.Spawn();

		Assert.ExpectError(TEXT("Tried to AddChildActorComponentTo"));
		Builder.AddChildActorComponentTo<AActor>();
	}
};
#endif