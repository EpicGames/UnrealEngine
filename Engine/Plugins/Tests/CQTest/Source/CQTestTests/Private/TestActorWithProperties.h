// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// -----------------------------------------------------------------------------

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"

#include "TestActorWithProperties.generated.h"

// -----------------------------------------------------------------------------

USTRUCT()
struct FTestStructWithProperties
{
	GENERATED_BODY()
public:
	UPROPERTY()
	int32 StructInt32Property = 0;

	UPROPERTY()
	TArray<int32> StructArrayProperty;
};

USTRUCT()
struct FDerivedTestStruct : public FTestStructWithProperties
{
	GENERATED_BODY()
public:
	int64 StructInt64Property = 0;
};

USTRUCT()
struct FOtherTestStruct
{
	GENERATED_BODY()
public:
	UPROPERTY()
	float StructFloatProperty = 0.0f;

	UPROPERTY()
	int64 StructInt64Property = 0;

	UPROPERTY()
	TMap<int32, int32> StructMapProperty;
};

UENUM()
enum class ETestUint8 : uint8
{
	enumone,
	enumtwo,
	enumthree
};
UENUM()
enum class ETestInt8 : int8
{
	enumone,
	enumtwo,
	enumthree
};
UENUM()
enum class ETestInt16 : int16
{
	enumone,
	enumtwo,
	enumthree
};
UENUM()
enum class ETestUint16 : uint16
{
	enumone,
	enumtwo,
	enumthree
};
UENUM()
enum class ETestInt32 : int32
{
	enumone,
	enumtwo,
	enumthree
};
UENUM()
enum class ETestUint32 : uint32
{
	enumone,
	enumtwo,
	enumthree
};
UENUM()
enum class ETestInt64 : int64
{
	enumone,
	enumtwo,
	enumthree
};
UENUM()
enum class ETestUint64 : uint64
{
	enumone,
	enumtwo,
	enumthree
};

UCLASS()
class ATestActorWithProperties : public AActor
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool BoolProperty;

	UPROPERTY()
	uint8 ByteProperty;

	UPROPERTY()
	int8 Int8Property;

	UPROPERTY()
	uint8 UInt8Property;

	UPROPERTY()
	uint16 UInt16Property;

	UPROPERTY()
	int16 Int16Property;

	UPROPERTY()
	uint32 UInt32Property;

	UPROPERTY()
	int64 Int64Property;

	UPROPERTY()
	uint64 UInt64Property;

	UPROPERTY()
	FTestStructWithProperties StructProperty;

	UPROPERTY()
	float FloatProperty;

	UPROPERTY()
	double DoubleProperty;

	UPROPERTY()
	int32 Int32Property;

	UPROPERTY()
	TObjectPtr<class UAnimSequence> TestTObjectPtrProperty;

	UPROPERTY()
	TScriptInterface<UObject> InterfaceProperty;

	UPROPERTY()
	FName NameProperty;

	UPROPERTY()
	TArray<int32> ArrayProperty;

	UPROPERTY()
	TMap<int32, int32> MapProperty;

	UPROPERTY()
	TSet<int32> SetProperty;

	UPROPERTY()
	FVector VectorProperty;

	UPROPERTY()
	ETestUint8 Uint8EnumProperty;

	UPROPERTY()
	ETestInt8 Int8EnumProperty;

	UPROPERTY()
	ETestInt16 Int16EnumProperty;

	UPROPERTY()
	ETestUint16 Uint16EnumProperty;

	UPROPERTY()
	ETestInt32 Int32EnumProperty;

	UPROPERTY()
	ETestUint32 Uint32EnumProperty;

	UPROPERTY()
	ETestInt64 Int64EnumProperty;

	UPROPERTY()
	ETestUint64 Uint64EnumProperty;

	UPROPERTY()
	TArray<TObjectPtr<ATestActorWithProperties>> ArrayOfObjectsProperty;

	UPROPERTY()
	TArray<FVector> ArrayOfVectorsProperty;

	UPROPERTY()
	TArray<FTestStructWithProperties> ArrayOfStructsProperty;
};

// -----------------------------------------------------------------------------
// -----------------------------------------------------------------------------
// -----------------------------------------------------------------------------
